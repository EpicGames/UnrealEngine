// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class PlanarCut : ModuleRules
	{
        public PlanarCut(ReadOnlyTargetRules Target) : base(Target)
		{
            PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"MeshUtilitiesCommon",
					"StaticMeshDescription",
					"MeshConversion",
					"Eigen",
				}
				);

			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"CoreUObject",
                    "Voronoi",
					"GeometryCore",
					"DynamicMesh",
					"Chaos",
					"GeometryAlgorithms",
					"MeshDescription",
                }
                );

			bDisableAutoRTFMInstrumentation = true;
		}
	}
}
