// Copyright Epic Games, Inc. All Rights Reserved.
#include "FractureAutoUV.h"
#include "PlanarCutPlugin.h"

#include "Distance/DistLine3Segment3.h"
#include "GeometryMeshConversion.h"


#include "Distance/DistLine3Triangle3.h"
#include "GeometryCollection/GeometryCollectionAlgo.h"

#include "Distance/DistSegment3Triangle3.h"
#include "DynamicMesh/MeshNormals.h"
#include "Generators/MeshShapeGenerator.h"
#include "Parameterization/DynamicMeshUVEditor.h"
#include "Sampling/MeshImageBakingCache.h"
#include "DynamicMesh/Operations/MergeCoincidentMeshEdges.h"

#include "Implicit/Solidify.h"

#include "Parameterization/PatchBasedMeshUVGenerator.h"
#include "Parameterization/UVPacking.h"
#include "Sampling/MeshCurvatureMapBaker.h"
#include "Sampling/MeshOcclusionMapBaker.h"
#include "Solvers/MeshSmoothing.h"
#include "DisjointSet.h"

#include "Image/ImageOccupancyMap.h"
#include "Spatial/FastWinding.h"
#include "Util/ProgressCancel.h"


#if WITH_EDITOR
#endif

using namespace UE::Geometry;



namespace UE
{
	namespace UVPacking
	{
		using namespace UE::Geometry;
		/**
		 * Create UV islands from a triangle mesh connectivity.
		 * Assumes the triangles are already split at UV seams, but topologically connected otherwise.
		 * Not recommended for meshes that already have built-in edge connectivity data.
		 *
		 * @param Mesh The mesh to create islands for
		 * @param IslandsOut The triangle IDs for each island
		 * @param IncludeTri Optional function to filter which tris are assigned to islands
		 */
		template <typename TriangleMeshType>
		inline void CreateUVIslandsFromMeshTopology(TriangleMeshType& Mesh,
			TArray<TArray<int>>& IslandsOut,
			TFunctionRef<bool(int32)> IncludeTri = [](int32)
		{
			return true;
		})
		{
			FDisjointSet VertComponents(Mesh.MaxVertexID());
			// Add Source vertices to hash & disjoint sets
			for (int32 TID = 0; TID < Mesh.MaxTriangleID(); TID++)
			{
				if (!Mesh.IsTriangle(TID) || !IncludeTri(TID))
				{
					continue;
				}

				FIndex3i Tri = Mesh.GetTriangle(TID);
				for (int32 First = 2, Second = 0; Second < 3; First = Second++)
				{
					VertComponents.Union(Tri[First], Tri[Second]);
				}
			}

			IslandsOut.Reset();
			TMap<uint32, int32> IslandIDToIdx;
			for (int32 TID = 0; TID < Mesh.MaxTriangleID(); TID++)
			{
				if (!Mesh.IsTriangle(TID) || !IncludeTri(TID))
				{
					continue;
				}

				FIndex3i Tri = Mesh.GetTriangle(TID);
				uint32 IslandID = VertComponents.Find(Tri.A);
				int32 Idx = -1;
				int32* FoundIdx = IslandIDToIdx.Find(IslandID);
				if (!FoundIdx)
				{
					Idx = IslandsOut.Emplace();
					IslandIDToIdx.Add(IslandID, Idx);
				}
				else
				{
					Idx = *FoundIdx;
				}
				IslandsOut[Idx].Add(TID);
			}
		}
	}
} // namespace UE::UVPacking

namespace UE::PrivateUVHelpers
{
	// Helper to update geometry UVs for meshes in specified bones via some processing that operates on a dynamic mesh
	bool ApplyDynamicMeshBasedUVProcessing(
		int32 TargetUVLayer, FGeometryCollection& Collection, 
		TArrayView<const int32> TransformIndices, TFunctionRef<bool(FDynamicMesh3&, int32 TransformIndex)> ProcessMesh,
		bool bWeldUVs = true, bool bSaveIsolatedVertices = true, bool bNeedsSplitAttributes = true)
	{
		using namespace UE::PlanarCut;

		int32 NumUVLayers = Collection.NumUVLayers();
		if (TargetUVLayer < 0 || TargetUVLayer >= NumUVLayers)
		{
			return false;
		}

		FDynamicMeshCollection CollectionMeshes(&Collection, TransformIndices, FTransform::Identity, bSaveIsolatedVertices);

		bool bAnyUpdate = false;

		for (int MeshIdx = 0; MeshIdx < CollectionMeshes.Meshes.Num(); MeshIdx++)
		{
			int32 TransformIdx = CollectionMeshes.Meshes[MeshIdx].TransformIndex;
			FDynamicMesh3& Mesh = CollectionMeshes.Meshes[MeshIdx].AugMesh;

			FMeshNormals::InitializeOverlayToPerVertexNormals(Mesh.Attributes()->PrimaryNormals(), true);
			AugmentedDynamicMesh::InitializeOverlayToPerVertexTangents(Mesh);
			AugmentedDynamicMesh::InitializeOverlayToPerVertexUVs(Mesh, NumUVLayers, 0);

			// Optionally weld edges so we won't have artificial UV seams on sharp edges, etc
			if (bWeldUVs)
			{
				FMergeCoincidentMeshEdges EdgeWelder(&Mesh);
				EdgeWelder.Apply();
			}

			if (ProcessMesh(Mesh, TransformIdx))
			{
				bAnyUpdate = true;
				Mesh.CompactInPlace();
				// Geometry collection requires split attributes, so we always need to re-split them if we've welded above.
				// We also re-split them if requested (i.e., if not welding, set this if ProcessMesh may create un-split vertices)
				if (bWeldUVs || bNeedsSplitAttributes)
				{
					AugmentedDynamicMesh::SplitOverlayAttributesToPerVertex(Mesh, true, true);
				}
			}
		}

		if (bAnyUpdate)
		{
			CollectionMeshes.UpdateAllCollections(Collection);
		}

		return bAnyUpdate;
	}
}

namespace UE { namespace PlanarCut {

namespace {
	inline bool IsTriActive(bool bIsVisible, bool bIsInternal, int32 MaterialID, const TSet<int32>& InsideMaterials, bool bActivateInsideTriangles, ETargetFaces TargetFaces)
	{
		if (!bIsVisible) // never select invisible triangles
		{
			return false;
		}

		if (TargetFaces == ETargetFaces::AllFaces && bActivateInsideTriangles)
		{
			return true;
		}
		if (
			(TargetFaces == ETargetFaces::InternalFaces && (bIsInternal == bActivateInsideTriangles)) ||
			(TargetFaces == ETargetFaces::ExternalFaces && (bIsInternal != bActivateInsideTriangles))
			)
		{
			return true;
		}
		if (InsideMaterials.Num() > 0 && (InsideMaterials.Contains(MaterialID) == bActivateInsideTriangles))
		{
			return true;
		}

		return false;
	}
}

/**
 * Shared helper method to set 'active' subset of triangles for a collection, to be considered for texturing
 * @return number of active triangles
 */
int32 SetActiveTriangles(const FGeometryCollection* Collection, TArray<bool>& ActiveTrianglesOut,
	bool bActivateInsideTriangles, ETargetFaces TargetFaces, TArrayView<int32> WhichMaterialsAreInside)
{
	TSet<int32> InsideMaterials;
	for (int32 MatID : WhichMaterialsAreInside)
	{
		InsideMaterials.Add(MatID);
	}

	ActiveTrianglesOut.Init(false, Collection->Indices.Num());
	int32 NumTriangles = 0;
	for (int TID = 0; TID < Collection->Indices.Num(); TID++)
	{
		bool bIsActive = IsTriActive(Collection->Visible[TID], Collection->Internal[TID], Collection->MaterialID[TID], InsideMaterials, bActivateInsideTriangles, TargetFaces);
		if (bIsActive)
		{
			ActiveTrianglesOut[TID] = true;
			NumTriangles++;
		}
	}

	return NumTriangles;
}

// Adapter to use geometry collections in the UVPacker, and other GeometricObjects generic mesh processing code
// Note: Adapts the whole geometry collection as a single mesh, with triangles filtered by an ActiveTriangles mask
struct FGeomMesh : public UE::Geometry::FUVPacker::IUVMeshView
{
	FGeometryCollection* Collection;
	const TArrayView<bool> ActiveTriangles;
	int32 NumTriangles;
	TArray<FVector3d> GlobalVertices; // vertices transformed to global space
	TArray<FVector3f> GlobalNormals; // normals transformed to global space
	int32 UVLayer = 0;
	TManagedArray<FVector2f>* CollectionUVs;

	// Construct a mesh from a geometry collection and a mask of active triangles
	FGeomMesh(int32 UVLayer, FGeometryCollection* Collection, TArrayView<bool> ActiveTriangles, int32 NumTriangles) 
		: Collection(Collection), ActiveTriangles(ActiveTriangles), NumTriangles(NumTriangles), UVLayer(UVLayer),
		CollectionUVs(Collection->FindUVLayer(UVLayer))
	{
		ValidateUVLayer();
		InitVertices();
	}

	// Construct a mesh from an existing FGeomMesh and a mask of active triangles
	FGeomMesh(const FGeomMesh& OtherMesh, TArrayView<bool> ActiveTriangles, int32 NumTriangles)
		: Collection(OtherMesh.Collection), ActiveTriangles(ActiveTriangles), NumTriangles(NumTriangles),
		  UVLayer(OtherMesh.UVLayer), CollectionUVs(OtherMesh.Collection->FindUVLayer(OtherMesh.UVLayer))
	{
		ValidateUVLayer();
		GlobalVertices = OtherMesh.GlobalVertices;
	}

	void ValidateUVLayer()
	{
		if (!ensure(UVLayer >= 0 && UVLayer < Collection->NumUVLayers()))
		{
			UVLayer = FMath::Clamp(UVLayer, 0, Collection->NumUVLayers() - 1);
			CollectionUVs = Collection->FindUVLayer(UVLayer);
		}
		check(CollectionUVs);
	}

	void InitVertices()
	{
		// transform all vertices from local to global space
		TArray<FTransform> GlobalTransformArray;
		GeometryCollectionAlgo::GlobalMatrices(Collection->Transform, Collection->Parent, GlobalTransformArray);
		GlobalVertices.SetNum(Collection->Vertex.Num());
		GlobalNormals.SetNum(Collection->Vertex.Num());
		for (int32 Idx = 0; Idx < GlobalVertices.Num(); Idx++)
		{
			GlobalNormals[Idx] = (FVector3f)GlobalTransformArray[Collection->BoneMap[Idx]].TransformVectorNoScale(FVector(Collection->Normal[Idx]));
			GlobalVertices[Idx] = (FVector3d)GlobalTransformArray[Collection->BoneMap[Idx]].TransformPosition(FVector(Collection->Vertex[Idx]));
		}
	}

	///
	/// The FUVPacker::IUVMeshView interface functions; this is the subset of functionality required for FUVPacker
	/// 

	virtual FIndex3i GetTriangle(int32 TID) const
	{
		return FIndex3i(Collection->Indices[TID]);
	}

	virtual FIndex3i GetUVTriangle(int32 TID) const
	{
		return FIndex3i(Collection->Indices[TID]);
	}

	virtual FVector3d GetVertex(int32 VID) const
	{
		return GlobalVertices[VID];
	}

	virtual FVector2f GetUV(int32 VID) const
	{
		return (*CollectionUVs)[VID];
	}

	virtual void SetUV(int32 VID, FVector2f UVIn)
	{
		FVector2f& UV = (*CollectionUVs)[VID];
		UV.X = UVIn.X;
		UV.Y = UVIn.Y;
	}



	///
	/// Interface for templated mesh code, not part of the the IUVMeshView interface
	///
	inline bool IsTriangle(int32 TID) const
	{
		return ActiveTriangles[TID];
	}
	constexpr inline bool IsVertex(int32 VID) const
	{
		return true; // we don't have sparse vertices, so can always return true
	}
	inline int32 MaxTriangleID() const
	{
		return Collection->Indices.Num();
	}
	inline int32 MaxVertexID() const
	{
		return GlobalVertices.Num();
	}
	inline int32 TriangleCount() const
	{
		return NumTriangles;
	}
	inline int32 VertexCount() const
	{
		return GlobalVertices.Num();
	}
	constexpr inline uint64 GetChangeStamp() const
	{
		return 1;
	}

	inline void GetTriVertices(int TID, UE::Math::TVector<double>& V0, UE::Math::TVector<double>& V1, UE::Math::TVector<double>& V2) const
	{
		FIntVector TriRaw = Collection->Indices[TID];

		V0 = GlobalVertices[TriRaw.X];
		V1 = GlobalVertices[TriRaw.Y];
		V2 = GlobalVertices[TriRaw.Z];
	}

	/// End of interfaces
	
	FVector3f GetInterpolatedNormal(int TID, const FVector3d& Bary)
	{
		FIntVector TriRaw = Collection->Indices[TID];
		FVector3f Normal = (FVector3f) (Bary.X * GlobalNormals[TriRaw.X] + Bary.Y * GlobalNormals[TriRaw.Y] + Bary.Z * GlobalNormals[TriRaw.Z]);
		return Normalized(Normal);
	}
};

/// Like FGeomMesh, but the UVs are given as the vertices rather than accessed separately
/// Used by the texture sampling code
struct FGeomFlatUVMesh
{
	FGeometryCollection* Collection;
	const TArrayView<bool> ActiveTriangles;
	int32 NumTriangles;
	int32 UVLayer;
	TManagedArray<FVector2f>* CollectionUVs;

	FGeomFlatUVMesh(int32 UVLayer, FGeometryCollection* Collection, TArrayView<bool> ActiveTriangles, int32 NumTriangles)
		: Collection(Collection), ActiveTriangles(ActiveTriangles), NumTriangles(NumTriangles), UVLayer(UVLayer), CollectionUVs(Collection->FindUVLayer(UVLayer))
	{
		ValidateUVLayer();
	}

	void ValidateUVLayer()
	{
		if (!ensure(UVLayer >= 0 && UVLayer < Collection->NumUVLayers()))
		{
			UVLayer = FMath::Clamp(UVLayer, 0, Collection->NumUVLayers());
			CollectionUVs = Collection->FindUVLayer(UVLayer);
		}
		check(CollectionUVs);
	}

	inline FIndex3i GetTriangle(int32 TID) const
	{
		return FIndex3i(Collection->Indices[TID]);
	}

	inline FVector3d GetVertex(int32 VID) const
	{
		const FVector2f& UV = (*CollectionUVs)[VID];
		return FVector3d(UV.X, UV.Y, 0);
	}

	inline bool IsTriangle(int32 TID) const
	{
		return ActiveTriangles[TID];
	}
	constexpr inline bool IsVertex(int32 VID) const
	{
		return true; // we don't have sparse vertices, so can always return true
	}
	inline int32 MaxTriangleID() const
	{
		return Collection->Indices.Num();
	}
	inline int32 MaxVertexID() const
	{
		return (*CollectionUVs).Num();
	}
	inline int32 TriangleCount() const
	{
		return NumTriangles;
	}
	inline int32 VertexCount() const
	{
		return (*CollectionUVs).Num();
	}
	constexpr inline uint64 GetChangeStamp() const
	{
		return 1;
	}

	inline void GetTriVertices(int TID, UE::Math::TVector<double>& V0, UE::Math::TVector<double>& V1, UE::Math::TVector<double>& V2) const
	{
		FIntVector TriRaw = Collection->Indices[TID];

		V0 = GetVertex(TriRaw.X);
		V1 = GetVertex(TriRaw.Y);
		V2 = GetVertex(TriRaw.Z);
	}
};


bool MergeUVIslands(
	int32 TargetUVLayer,
	FGeometryCollection& Collection,
	FMergeIslandSettings MergeIslandSettings,
	TArrayView<bool> FaceSelection,
	FProgressCancel* Progress
)
{
	TSet<int32> ActiveTransformIndexSet;
	for (int32 FaceIdx = 0; FaceIdx < FaceSelection.Num(); ++FaceIdx)
	{
		ActiveTransformIndexSet.Add(Collection.BoneMap[Collection.Indices[FaceIdx].X]);
	}
	TArray<int32> TransformIndices = ActiveTransformIndexSet.Array();

	return UE::PrivateUVHelpers::ApplyDynamicMeshBasedUVProcessing(
		TargetUVLayer, Collection,
		TransformIndices, [&MergeIslandSettings, &FaceSelection, &Collection, TargetUVLayer](FDynamicMesh3& Mesh, int32 TransformIndex) -> bool
		{

			FDynamicMeshUVOverlay* UVLayer = Mesh.Attributes()->GetUVLayer(TargetUVLayer);
			if (!ensure(UVLayer))
			{
				return false;
			}

			// Pull the selected faces for the current mesh from the overall FaceSelection, via the mesh TransformIndex
			int32 GeometryIndex = Collection.TransformToGeometryIndex[TransformIndex];
			int32 FaceStart = Collection.FaceStart[GeometryIndex];
			// Note: Mesh must have the same triangle count as the corresponding geometry, and the triangles must be compact
			if (!ensure(Collection.FaceCount[GeometryIndex] == Mesh.TriangleCount() && Mesh.IsCompactT()))
			{
				return false;
			}
			TArray<int32> TargetTris;
			for (int TID : Mesh.TriangleIndicesItr())
			{
				if (FaceSelection[TID + FaceStart])
				{
					TargetTris.Add(TID);
				}
			}
			if (TargetTris.IsEmpty())
			{
				return false;
			}
			FPatchBasedMeshUVGenerator PatchUVGen;
			PatchUVGen.MergingThreshold = MergeIslandSettings.AreaDistortionThreshold;
			PatchUVGen.MaxNormalDeviationDeg = MergeIslandSettings.MaxNormalDeviationDeg;
			PatchUVGen.NormalSmoothingAlpha = MergeIslandSettings.NormalSmoothingAlpha;
			PatchUVGen.NormalSmoothingRounds = MergeIslandSettings.NormalSmoothingRounds;
			FMeshConnectedComponents InitialComponents(&Mesh);
			InitialComponents.FindConnectedTriangles(TargetTris, [&Mesh, &UVLayer](int32 TA, int32 TB) -> bool
				{
					return UVLayer->AreTrianglesConnected(TA, TB);
				}
			);
			TArray<TArray<int32>> MergedIslands;
			PatchUVGen.ComputeIslandsByRegionMerging(Mesh, *UVLayer, InitialComponents, true, MergedIslands);
			if (MergedIslands.IsEmpty())
			{
				// nothing merged, no need update
				return false;
			}

			TArray<double> OriginalUVAreas;
			TArray<FVector2f> RefUVPos;
			OriginalUVAreas.SetNumZeroed(MergedIslands.Num());
			RefUVPos.SetNumZeroed(MergedIslands.Num());
			for (int32 IslandIdx = 0; IslandIdx < MergedIslands.Num(); ++IslandIdx)
			{
				OriginalUVAreas[IslandIdx] = FDynamicMeshUVEditor::DetermineAreaFromUVs(*UVLayer, MergedIslands[IslandIdx]);
				// Find any valid UV as a reference position
				for (int32 TID : MergedIslands[IslandIdx])
				{
					FIndex3i ElTri = UVLayer->GetTriangle(TID);
					if (ElTri.A != INDEX_NONE)
					{
						RefUVPos[IslandIdx] = UVLayer->GetElement(ElTri.A);
						break;
					}
				}
			}

			TArray<bool> ValidUVsComputed;
			int32 NumSolvesFailed = PatchUVGen.ComputeUVsFromTriangleSets(Mesh, *UVLayer, MergedIslands, ValidUVsComputed);

			for (int32 IslandIdx = 0; IslandIdx < MergedIslands.Num(); ++IslandIdx)
			{
				double NewAreaSum = FDynamicMeshUVEditor::DetermineAreaFromUVs(*UVLayer, MergedIslands[IslandIdx]);
				if (OriginalUVAreas[IslandIdx] > 0 && NewAreaSum > 0)
				{
					float Scale = (float)FMath::Sqrt(OriginalUVAreas[IslandIdx] / NewAreaSum);
					FVector2f Origin = RefUVPos[IslandIdx];
					FDynamicMeshUVEditor::TransformTriangleSelectionUVs(*UVLayer, MergedIslands[IslandIdx], [Scale, Origin](const FVector2f& UV)
					{
						return (UV - Origin) * Scale + Origin;
					});
				}
			}

			return true;
		},
		true
	);
}


bool BoxProjectUVs(
	int32 TargetUVLayer,
	FGeometryCollection& Collection,
	const FVector3d& BoxDimensions,
	ETargetFaces TargetFaces,
	TArrayView<int32> WhichMaterials,
	FVector2f OffsetUVs,
	bool bOverrideBoxDimensionsWithBounds,
	bool bCenterBoxAtPivot,
	bool bUniformProjectionScale
)
{
	TArray<int32> TransformIndices;
	for (int32 GeomIdx = 0; GeomIdx < Collection.TransformIndex.Num(); GeomIdx++)
	{
		TransformIndices.Add(Collection.TransformIndex[GeomIdx]);
	}
	TSet<int32> TargetMaterials;
	TargetMaterials.Append(WhichMaterials);

	return UE::PrivateUVHelpers::ApplyDynamicMeshBasedUVProcessing(
		TargetUVLayer, Collection,
		TransformIndices, 
		[&TargetMaterials, TargetUVLayer, &BoxDimensions, TargetFaces, &WhichMaterials, &OffsetUVs,
		bOverrideBoxDimensionsWithBounds, bCenterBoxAtPivot, bUniformProjectionScale]
		(FDynamicMesh3& Mesh, int32 TransformIndex) -> bool
		{
			TArray<int32> TargetTris;
			for (int TID : Mesh.TriangleIndicesItr())
			{
				bool bIsTextureTri = IsTriActive(
					AugmentedDynamicMesh::GetVisibility(Mesh, TID), AugmentedDynamicMesh::GetInternal(Mesh, TID),
					Mesh.Attributes()->GetMaterialID()->GetValue(TID), TargetMaterials, true, TargetFaces);
				if (bIsTextureTri)
				{
					TargetTris.Add(TID);
				}
			}
			if (TargetTris.IsEmpty())
			{
				return false;
			}

			FDynamicMeshUVEditor UVEd(&Mesh, TargetUVLayer, false);
			FFrame3d BoxFrame; // defaults to origin / no rotation
			FVector3d UseBoxDimensions = bUniformProjectionScale ? FVector3d(BoxDimensions.X) : BoxDimensions;
			if (!bCenterBoxAtPivot || bOverrideBoxDimensionsWithBounds)
			{
				FAxisAlignedBox3d Bounds = Mesh.GetBounds(true);
				if (!bCenterBoxAtPivot)
				{
					BoxFrame.Origin = Bounds.Center();
				}
				if (bOverrideBoxDimensionsWithBounds)
				{
					UseBoxDimensions = bUniformProjectionScale ? FVector3d(Bounds.MaxDim()) : Bounds.Max - Bounds.Min;
				}
			}
			UVEd.SetTriangleUVsFromBoxProjection(TargetTris, [](const FVector3d& Pos) { return Pos; }, BoxFrame, UseBoxDimensions, 1);

			// apply offset to UV space
			FDynamicMeshUVOverlay* Overlap = UVEd.GetOverlay();
			for (int ElemID : Overlap->ElementIndicesItr())
			{
				Overlap->SetElement(ElemID, OffsetUVs + Overlap->GetElement(ElemID));
			}
			return true;
		},
		true);
}


bool UVLayout(
	int32 TargetUVLayer,
	FGeometryCollection& Collection,
	int32 UVRes,
	float GutterSize,
	ETargetFaces TargetFaces,
	TArrayView<int32> WhichMaterials,
	bool bRecreateUVsForDegenerateIslands,
	FProgressCancel* Progress
)
{
	TArray<bool> FaceSelection;
	int32 NumActive = SetActiveTriangles(&Collection, FaceSelection, true, TargetFaces, WhichMaterials);
	return UVLayout(TargetUVLayer, Collection, UVRes, GutterSize, FaceSelection, bRecreateUVsForDegenerateIslands, Progress);
}


bool PLANARCUT_API UVLayout(
	int32 TargetUVLayer,
	FGeometryCollection& Collection,
	int32 UVRes,
	float GutterSize,
	TArrayView<bool> FaceSelection,
	bool bRecreateUVsForDegenerateIslands,
	FProgressCancel* Progress
)
{
	// TargetUVLayer must be valid for the collection
	if (TargetUVLayer < 0 || TargetUVLayer >= GeometryCollectionUV::MAX_NUM_UV_CHANNELS || !Collection.FindUVLayer(TargetUVLayer))
	{
		UE_LOGF(LogPlanarCut, Error, "UV Layout failed: Target UV Channel (%d) not found on collection", TargetUVLayer);
		return false;
	}
	// Face selection array size must match the collection face count
	if (Collection.Indices.Num() != FaceSelection.Num())
	{
		UE_LOGF(LogPlanarCut, Error, "UV Layout failed: Mismatch between collection face count (%d) and selection face count (%d)", Collection.Indices.Num(), FaceSelection.Num());
		return false;
	}

	FProgressCancel::FProgressScope InitScope = FProgressCancel::CreateScopeTo(Progress, .2f);

	int32 NumActive = 0;
	for (const bool& bSelected : FaceSelection)
	{
		NumActive += (bSelected) ? 1 : 0;
	}
	FGeomMesh UVMesh(TargetUVLayer, &Collection, FaceSelection, NumActive);

	TArray<TArray<int32>> UVIslands;
	UE::UVPacking::CreateUVIslandsFromMeshTopology<FGeomMesh>(UVMesh, UVIslands);

	InitScope.Done();

	if (Progress && Progress->Cancelled())
	{
		return false;
	}

	if (bRecreateUVsForDegenerateIslands)
	{
		FProgressCancel::FProgressScope ForDegenerateIslandsScope = FProgressCancel::CreateScopeTo(Progress, .5);

		TArray<int> IslandVert; IslandVert.SetNumUninitialized(UVMesh.MaxVertexID());

		const int32 NumIslands = UVIslands.Num();
		for (int IslandIdx = 0; IslandIdx < NumIslands; IslandIdx++)
		{
			double UVArea = 0;
			FVector3d AvgNormal(0, 0, 0);
			FVector3d AnyPt(0, 0, 0);
			for (int TID : UVIslands[IslandIdx])
			{
				FIndex3i UVInds = UVMesh.GetUVTriangle(TID);
				FVector2f UVA = UVMesh.GetUV(UVInds.A), UVB = UVMesh.GetUV(UVInds.B), UVC = UVMesh.GetUV(UVInds.C);
				UVArea += (double)VectorUtil::Area(UVA, UVB, UVC);
				FVector3d VA, VB, VC;
				UVMesh.GetTriVertices(TID, VA, VB, VC);
				double Area;
				AnyPt = VA;
				FVector3d Normal = VectorUtil::NormalArea(VA, VB, VC, Area);
				AvgNormal += Area * Normal;
			}
			double AvgNormalLen = Normalize(AvgNormal);
			bool bHasValidNormal = AvgNormalLen > 0;
			bool bDoProjection = FMathd::Abs(UVArea) < FMathd::ZeroTolerance;
			if (bDoProjection)
			{
				// convert the island to a dynamic mesh so we can use the expmap UVs
				for (int VID = 0; VID < IslandVert.Num(); ++VID)
				{
					IslandVert[VID] = -1;
				}
				TArray<int> InvIslandVert; InvIslandVert.Reserve(3 * UVIslands[IslandIdx].Num());
				FDynamicMesh3 IslandMesh;
				for (int TID : UVIslands[IslandIdx])
				{
					FIndex3i Tri = UVMesh.GetUVTriangle(TID);
					FIndex3i NewTri = FIndex3i::Invalid();
					for (int SubIdx = 0; SubIdx < 3; SubIdx++)
					{
						int SrcVID = Tri[SubIdx];
						int& VID = IslandVert[SrcVID];
						if (VID == -1)
						{
							VID = IslandMesh.AppendVertex(UVMesh.GetVertex(SrcVID));
							check(InvIslandVert.Num() == VID);
							InvIslandVert.Add(SrcVID);
						}
						NewTri[SubIdx] = VID;
					}
					IslandMesh.AppendTriangle(NewTri);
				}
				FDynamicMeshUVEditor UVEditor(&IslandMesh, 0, true);
				TArray<int> Tris; Tris.Reserve(IslandMesh.TriangleCount());
				for (int TID : IslandMesh.TriangleIndicesItr())
				{
					Tris.Add(TID);
				}
				bool bExpMapSuccess = UVEditor.SetTriangleUVsFromExpMap(Tris);
				if (bExpMapSuccess)
				{
					// transfer UVs from the overlay
					FDynamicMeshUVOverlay* UVOverlay = IslandMesh.Attributes()->PrimaryUV();
					for (int IslandTID : IslandMesh.TriangleIndicesItr())
					{
						FIndex3i Tri = IslandMesh.GetTriangle(IslandTID);
						FIndex3i UVTri = UVOverlay->GetTriangle(IslandTID);
						if (!ensure(UVTri != FIndex3i::Invalid()))
						{
							// we shouldn't have unset tris like this if the ExpMap returned success
							bExpMapSuccess = false;
							break;
						}
						for (int SubIdx = 0; SubIdx < 3; SubIdx++)
						{
							int VID = InvIslandVert[Tri[SubIdx]];
							FVector2f UV = UVOverlay->GetElement(UVTri[SubIdx]);
							UVMesh.SetUV(VID, UV);
						}
					}
				}
				if (!bExpMapSuccess && bHasValidNormal)
				{
					// expmap failed; fall back to projecting UVs
					FFrame3d Frame(AnyPt, AvgNormal);
					for (int TID : UVIslands[IslandIdx])
					{
						FIndex3i Tri = UVMesh.GetUVTriangle(TID);
						for (int SubIdx = 0; SubIdx < 3; SubIdx++)
						{
							int VID = Tri[SubIdx];
							FVector2f UV = (FVector2f)Frame.ToPlaneUV(UVMesh.GetVertex(VID));
							UVMesh.SetUV(VID, UV);
						}
					}
				}
			}
		}
	}

	if (Progress && Progress->Cancelled())
	{
		return false;
	}
	FProgressCancel::FProgressScope FinalScope = FProgressCancel::CreateScopeTo(Progress, 1);

	UE::Geometry::FUVPacker Packer;
	Packer.bScaleIslandsByWorldSpaceTexelRatio = true; // let packer scale islands separately to have consistent texel-to-world ratio
	Packer.bAllowFlips = false;
	// StandardPack doesn't support the Packer.GutterSize feature, and always tries to leave a 1 texel gutter.
	// To approximate a larger gutter, we tell it to consider a smaller output resolution --
	//  hoping that e.g. the UVs for a 1 pixel gutter at 256x256 are ~ a 2 pixel gutter at 512x512
	// TODO: If we make StandardPack support the GutterSize parameter, we can use that instead.
	Packer.TextureResolution = static_cast<int32>( UVRes / FMathf::Max(1.f, GutterSize) );
	return Packer.StandardPack(&UVMesh, UVIslands);
}


bool TextureInternalSurfaces(
	int32 TargetUVLayer,
	FGeometryCollection& Collection,
	int32 GutterSize,
	UE::Geometry::FIndex4i BakeAttributes,
	const FTextureAttributeSettings& AttributeSettings,
	UE::Geometry::TImageBuilder<FVector4f>& TextureOut,
	EUseMaterials MaterialsPattern,
	TArrayView<int32> WhichMaterials,
	FProgressCancel* Progress
)
{
	ETargetFaces TargetFaces =
		MaterialsPattern == EUseMaterials::AllMaterials ? ETargetFaces::AllFaces :
		MaterialsPattern == EUseMaterials::OddMaterials ? ETargetFaces::InternalFaces : ETargetFaces::CustomFaces;
	return TextureSpecifiedFaces(TargetUVLayer, Collection, GutterSize, BakeAttributes, AttributeSettings, TextureOut, TargetFaces, WhichMaterials, Progress);
}



bool TextureSpecifiedFaces(
	int32 TargetUVLayer,
	FGeometryCollection& Collection,
	int32 GutterSize,
	FIndex4i BakeAttributes,
	const FTextureAttributeSettings& AttributeSettings,
	TImageBuilder<FVector4f>& TextureOut,
	ETargetFaces TargetFaces,
	TArrayView<int32> WhichMaterials,
	FProgressCancel* Progress
)
{
	TArray<bool> ToTextureTriangles;
	SetActiveTriangles(&Collection, ToTextureTriangles, true, TargetFaces, WhichMaterials);

	return TextureSpecifiedFaces(
		TargetUVLayer,
		Collection,
		GutterSize,
		BakeAttributes,
		AttributeSettings,
		TextureOut,
		ToTextureTriangles,
		Progress);
}

bool TextureSpecifiedFaces(
	int32 TargetUVLayer,
	FGeometryCollection& Collection,
	int32 GutterSize,
	FIndex4i BakeAttributes,
	const FTextureAttributeSettings& AttributeSettings,
	TImageBuilder<FVector4f>& TextureOut,
	TArrayView<bool> ToTextureTriangles,
	FProgressCancel* Progress
)
{
	int32 NumTextureTris = 0;
	for (const bool& bSelected : ToTextureTriangles)
	{
		NumTextureTris += (bSelected) ? 1 : 0;
	}

	FGeomFlatUVMesh UVMesh(TargetUVLayer, &Collection, ToTextureTriangles, NumTextureTris);

	FImageOccupancyMap OccupancyMap;
	OccupancyMap.GutterSize = GutterSize;
	OccupancyMap.Initialize(TextureOut.GetDimensions());
	OccupancyMap.ComputeFromUVSpaceMesh(UVMesh, [](int32 TriangleID) { return TriangleID; });

	FGeomMesh ToTextureMesh(TargetUVLayer, &Collection, ToTextureTriangles, NumTextureTris);

	// Collect the external faces
	TArray<bool> OutsideTriangles;
	int32 NumOutsideTris = SetActiveTriangles(&Collection, OutsideTriangles, true, ETargetFaces::ExternalFaces, {});
	FGeomMesh OutsideMesh(ToTextureMesh, OutsideTriangles, NumOutsideTris);
	TMeshAABBTree3<FGeomMesh> OutsideSpatial(&OutsideMesh);

	int DistanceToExternalIdx = BakeAttributes.IndexOf((int)EBakeAttributes::DistanceToExternal);
	int AmbientIdx = BakeAttributes.IndexOf((int)EBakeAttributes::AmbientOcclusion);
	int CurvatureIdx = BakeAttributes.IndexOf((int)EBakeAttributes::Curvature);
	int NormalXIdx = BakeAttributes.IndexOf((int)EBakeAttributes::NormalX);
	int NormalYIdx = BakeAttributes.IndexOf((int)EBakeAttributes::NormalY);
	int NormalZIdx = BakeAttributes.IndexOf((int)EBakeAttributes::NormalZ);

	FAxisAlignedBox3d Box = OutsideSpatial.GetBoundingBox();
	int PosXIdx = Box.Width() > 0 ? BakeAttributes.IndexOf((int)EBakeAttributes::PositionX) : -1;
	int PosYIdx = Box.Height() > 0 ? BakeAttributes.IndexOf((int)EBakeAttributes::PositionY) : -1;
	int PosZIdx = Box.Depth() > 0 ? BakeAttributes.IndexOf((int)EBakeAttributes::PositionZ) : -1;

	bool bNeedsDynamicMeshes = AmbientIdx > -1 || CurvatureIdx > -1;

	TArray<int32> TransformIndices;
	if (bNeedsDynamicMeshes)
	{
		for (int32 GeomIdx = 0; GeomIdx < Collection.TransformIndex.Num(); GeomIdx++)
		{
			int32 TransformIdx = Collection.TransformIndex[GeomIdx];
			if (Collection.IsVisible(TransformIdx))
			{
				TransformIndices.Add(TransformIdx);
			}
		}
	}

	const float AmbientWorkPer = 1, CurvatureWorkPer = 10;
	const int32 NumMeshes = TransformIndices.Num();
	const float AmountOfWork = 
		float(AmbientIdx > -1) * AmbientWorkPer * NumMeshes +
		float(CurvatureIdx > -1) * CurvatureWorkPer * NumMeshes;
	const float WorkIncr = AmountOfWork > 0 ? 0.9f / AmountOfWork : 0.f;

	FDynamicMeshCollection CollectionMeshes;
	CollectionMeshes.bGenerateMeshToCollectionFaceMapping = true;
	CollectionMeshes.Init(&Collection, TransformIndices, FTransform::Identity, false);
	if (bNeedsDynamicMeshes)
	{
		// To bake only selected faces, unset texture on everything else
		for (int MeshIdx = 0; MeshIdx < CollectionMeshes.Meshes.Num(); MeshIdx++)
		{
			FDynamicMeshCollection::FMeshData& MeshData = CollectionMeshes.Meshes[MeshIdx];
			int32 TransformIdx = MeshData.TransformIndex;
			FDynamicMesh3& Mesh = MeshData.AugMesh;
			FMeshNormals::InitializeOverlayToPerVertexNormals(Mesh.Attributes()->PrimaryNormals(), true);
			AugmentedDynamicMesh::InitializeOverlayToPerVertexUVs(Mesh, 1, TargetUVLayer); // build an overlay w/ *only* the uv layer we care about
			FDynamicMeshUVOverlay* UV = Mesh.Attributes()->PrimaryUV();
			for (int TID : Mesh.TriangleIndicesItr())
			{
				const int32 CollectionFaceIndex = MeshData.GetCollectionFaceFromMeshFace(TID);
				if (ensure(CollectionFaceIndex != INDEX_NONE && ToTextureTriangles.IsValidIndex(CollectionFaceIndex)))
				{
					if (ToTextureTriangles[CollectionFaceIndex] == false)
					{
						UV->UnsetTriangle(TID);
					}
				}
			}
		}
	}

	if (Progress && Progress->Cancelled())
	{
		return false;
	}

	// Gutter for separate bakes used for curvature / AO / etc -- we shouldn't need the full gutter step for these,
	//  because we do a post-process later to fill the gutter for the whole image.
	// But we still need a positive gutter size for the bakes to work, so we use a size of 1.
	const int32 PartialBakeGutterSize = 1;

	TArray64<float> CurvatureValues; // buffer to fill with curvature values
	if (CurvatureIdx > -1)
	{
		int UseVoxRes = FMath::Max(8, AttributeSettings.Curvature_VoxelRes);
		FImageDimensions Dimensions = TextureOut.GetDimensions();
		int64 TexelsNum = Dimensions.Num();
		CurvatureValues.SetNumZeroed(TexelsNum);
		for (int MeshIdx = 0; MeshIdx < CollectionMeshes.Meshes.Num(); MeshIdx++)
		{
			if (Progress && Progress->Cancelled())
			{
				return false;
			}
			FProgressCancel::FProgressScope CurvatureScope(Progress, CurvatureWorkPer * WorkIncr);

			int32 TransformIdx = CollectionMeshes.Meshes[MeshIdx].TransformIndex;
			FDynamicMesh3& Mesh = CollectionMeshes.Meshes[MeshIdx].AugMesh;

			FDynamicMeshAABBTree3 Spatial(&Mesh);
			TFastWindingTree<FDynamicMesh3> FastWinding(&Spatial);
			double ExtendBounds = Spatial.GetBoundingBox().MaxDim() * .01;

			TImplicitSolidify<FDynamicMesh3> Solidify(&Mesh, &Spatial, &FastWinding);
			Solidify.SetCellSizeAndExtendBounds(Spatial.GetBoundingBox(), ExtendBounds, UseVoxRes);
			Solidify.WindingThreshold = AttributeSettings.Curvature_Winding;
			Solidify.SurfaceSearchSteps = 3;
			Solidify.bSolidAtBoundaries = true;
			Solidify.ExtendBounds = ExtendBounds;

			FDynamicMesh3 SolidMesh(&Solidify.Generate());

			// Smooth mesh
			double SmoothAlpha = AttributeSettings.Curvature_SmoothingPerStep;
			TArray<FVector3d> PositionBuffer;
			PositionBuffer.SetNumUninitialized(SolidMesh.MaxVertexID());
			for (int VID : SolidMesh.VertexIndicesItr())
			{
				PositionBuffer[VID] = SolidMesh.GetVertex(VID);
			}
			UE::MeshDeformation::ComputeSmoothing_Forward(true, false, SolidMesh, [SmoothAlpha](int VID, bool bIsBoundary) { return SmoothAlpha; },
				AttributeSettings.Curvature_SmoothingSteps, PositionBuffer);
			for (int VID : SolidMesh.VertexIndicesItr())
			{
				SolidMesh.SetVertex(VID, PositionBuffer[VID]);
			}

			FDynamicMeshAABBTree3 SolidSpatial(&SolidMesh);

			FMeshImageBakingCache BakeCache;
			BakeCache.SetGutterSize(PartialBakeGutterSize);
			BakeCache.SetDetailMesh(&SolidMesh, &SolidSpatial);
			BakeCache.SetBakeTargetMesh(&Mesh);
			BakeCache.SetDimensions(Dimensions);
			BakeCache.SetUVLayer(0);
			// set distance to search for correspondences based on the voxel size
			double Thickness = AttributeSettings.Curvature_ThicknessFactor * Spatial.GetBoundingBox().MaxDim() / (float)UseVoxRes;
			BakeCache.SetThickness(Thickness);
			BakeCache.ValidateCache();

			FMeshCurvatureMapBaker CurvatureBaker;
			CurvatureBaker.bOverrideCurvatureRange = true;
			CurvatureBaker.OverrideRangeMax = AttributeSettings.Curvature_MaxValue;
			CurvatureBaker.UseColorMode = FMeshCurvatureMapBaker::EColorMode::BlackGrayWhite;
			CurvatureBaker.SetCache(&BakeCache);

			CurvatureBaker.BlurRadius = AttributeSettings.bCurvature_Blur ? AttributeSettings.Curvature_BlurRadius : 0.0;

			CurvatureBaker.Bake();

			const TUniquePtr<TImageBuilder<FVector3f>>& Result = CurvatureBaker.GetResult();
			// copy scalar curvature values out to the accumulated curvature buffer
			for (int64 LinearIdx = 0; LinearIdx < TexelsNum; LinearIdx++)
			{
				float& CurvatureOut = CurvatureValues[LinearIdx];
				CurvatureOut = FMathf::Max(CurvatureOut, Result->GetPixel(LinearIdx).X);
			}
		}
	}
	
	TArray64<float> AmbientValues; // buffer to fill with ambient occlusion values
	if (AmbientIdx > -1)
	{
		FImageDimensions Dimensions = TextureOut.GetDimensions();
		int64 TexelsNum = Dimensions.Num();
		AmbientValues.Init(1.0f, TexelsNum);
		for (int MeshIdx = 0; MeshIdx < CollectionMeshes.Meshes.Num(); MeshIdx++)
		{
			if (Progress && Progress->Cancelled())
			{
				return false;
			}
			FProgressCancel::FProgressScope CurvatureScope(Progress, AmbientWorkPer * WorkIncr);

			int32 TransformIdx = CollectionMeshes.Meshes[MeshIdx].TransformIndex;
			FDynamicMesh3& Mesh = CollectionMeshes.Meshes[MeshIdx].AugMesh;

			FDynamicMeshAABBTree3 Spatial(&Mesh);

			FMeshImageBakingCache BakeCache;
			BakeCache.SetGutterSize(PartialBakeGutterSize);
			BakeCache.SetDetailMesh(&Mesh, &Spatial);
			BakeCache.SetBakeTargetMesh(&Mesh);
			BakeCache.SetDimensions(Dimensions);
			BakeCache.SetUVLayer(0);
			BakeCache.SetCorrespondenceStrategy(FMeshImageBakingCache::ECorrespondenceStrategy::Identity);
			// thickness shouldn't matter because we're using Identity ECorrespondenceStrategy::Identity
			BakeCache.SetThickness(KINDA_SMALL_NUMBER);
			BakeCache.ValidateCache();

			FMeshOcclusionMapBaker OcclusionBaker;
			OcclusionBaker.SetCache(&BakeCache);
			
			OcclusionBaker.OcclusionType = EOcclusionMapType::AmbientOcclusion;
			OcclusionBaker.NumOcclusionRays = AttributeSettings.AO_Rays;
			OcclusionBaker.MaxDistance = AttributeSettings.AO_MaxDistance == 0 ? TNumericLimits<double>::Max() : AttributeSettings.AO_MaxDistance;
			OcclusionBaker.BiasAngleDeg = AttributeSettings.AO_BiasAngleDeg;
			OcclusionBaker.BlurRadius = AttributeSettings.bAO_Blur ? AttributeSettings.AO_BlurRadius : 0.0;

			OcclusionBaker.Bake();

			const TUniquePtr<TImageBuilder<FVector3f>>& Result = OcclusionBaker.GetResult(FMeshOcclusionMapBaker::EResult::AmbientOcclusion);
			// copy scalar ambient values out to the accumulated ambient buffer
			for (int64 LinearIdx = 0; LinearIdx < TexelsNum; LinearIdx++)
			{
				float& AmbientOut = AmbientValues[LinearIdx];
				AmbientOut = FMathf::Min(AmbientOut, Result->GetPixel(LinearIdx).X);
			}
		}
	}

	if (Progress && Progress->Cancelled())
	{
		return false;
	}
	FProgressCancel::FProgressScope EndBake = FProgressCancel::CreateScopeTo(Progress, 1);

	ParallelFor(OccupancyMap.Dimensions.GetHeight(),
		[&AttributeSettings, &TextureOut, &UVMesh, &OccupancyMap, &ToTextureMesh, &OutsideSpatial,
		 &DistanceToExternalIdx, &AmbientIdx, &NormalXIdx, &NormalYIdx, &NormalZIdx, &PosXIdx, &PosYIdx, &PosZIdx](int32 Y)
	{
		for (int32 X = 0; X < OccupancyMap.Dimensions.GetWidth(); X++)
		{
			int64 LinearCoord = OccupancyMap.Dimensions.GetIndex(X, Y);
			if (OccupancyMap.IsInterior(LinearCoord))
			{
				FVector4f OutColor = FVector4f::One();

				int32 TID = OccupancyMap.TexelQueryTriangle[LinearCoord];
				FVector2f UV = OccupancyMap.TexelQueryUV[LinearCoord];
				// Note this is the slowest way to get barycentric coordinates out of a point and a 2D triangle,
				//   ... but it's the one we currently have that's robust to (and gives good answers on) degenerate triangles
				FDistPoint3Triangle3d DistQuery = TMeshQueries<FGeomFlatUVMesh>::TriangleDistance(UVMesh, TID, FVector3d(UV.X, UV.Y, 0));
				FVector3d Bary = DistQuery.TriangleBaryCoords;
				FVector3f Normal(0, 0, 1);
				bool bNeedsNormal = NormalXIdx > -1 || NormalYIdx > -1 || NormalZIdx > -1 || AmbientIdx > -1;
				if (bNeedsNormal)
				{
					Normal = ToTextureMesh.GetInterpolatedNormal(TID, Bary);
				}

				if (DistanceToExternalIdx > -1 || PosXIdx > -1 || PosYIdx > -1 || PosZIdx > -1)
				{
					FTriangle3d Tri;
					ToTextureMesh.GetTriVertices(TID, Tri.V[0], Tri.V[1], Tri.V[2]);
					FVector3d InsidePoint = Tri.BarycentricPoint(Bary);

					if (DistanceToExternalIdx > -1)
					{
						double DistanceSq;
						OutsideSpatial.FindNearestTriangle(InsidePoint, DistanceSq, AttributeSettings.ToExternal_MaxDistance);
						float PercentDistance = FMathf::Min(1.0f, static_cast<float>(FMathd::Sqrt(DistanceSq) / AttributeSettings.ToExternal_MaxDistance));
						checkSlow(FMath::IsFinite(PercentDistance));
						OutColor[DistanceToExternalIdx] = PercentDistance;
					}
					auto SetSpatial = [&OutsideSpatial, &InsidePoint, &OutColor](int TargetIdx, int Dim)
					{
						if (TargetIdx > -1)
						{
							double Min = OutsideSpatial.GetBoundingBox().Min[Dim], Max = OutsideSpatial.GetBoundingBox().Max[Dim];
							checkSlow(Min != Max);
							float PercentAlong = static_cast<float>( (InsidePoint[Dim] - Min) / (Max - Min) );
							OutColor[TargetIdx] = PercentAlong;
						}
					};
					SetSpatial(PosXIdx, 0);
					SetSpatial(PosYIdx, 1);
					SetSpatial(PosZIdx, 2);
				}
				auto SetNormal = [&Normal, &OutColor](int TargetIdx, int Dim)
				{
					if (TargetIdx > -1)
					{
						float NormalVal = Normal[Dim];
						{
							NormalVal = (1 + NormalVal) * .5f; // compress from [-1,1] to [0,1] range
						}
						OutColor[TargetIdx] = NormalVal;
					}
				};
				SetNormal(NormalXIdx, 0);
				SetNormal(NormalYIdx, 1);
				SetNormal(NormalZIdx, 2);

				TextureOut.SetPixel(LinearCoord, OutColor);
			}
		}
	});

	if (Progress && Progress->Cancelled())
	{
		return false;
	}

	auto CopyValuesToChannel = [&TextureOut, &OccupancyMap](int ChannelIdx, TArray64<float>& Values)
	{
		if (ChannelIdx > -1)
		{
			int64 TexelsNum = OccupancyMap.Dimensions.Num();
			for (int64 LinearIdx = 0; LinearIdx < TexelsNum; LinearIdx++)
			{
				FVector4f Pixel = TextureOut.GetPixel(LinearIdx);
				Pixel[ChannelIdx] = Values[LinearIdx];
				TextureOut.SetPixel(LinearIdx, Pixel);
			}
		}
	};

	CopyValuesToChannel(AmbientIdx, AmbientValues);
	CopyValuesToChannel(CurvatureIdx, CurvatureValues);

	if (AttributeSettings.ClearGutterChannel > -1 && AttributeSettings.ClearGutterChannel < 4)
	{
		for (TTuple<int64, int64>& GutterToInside : OccupancyMap.GutterTexels)
		{
			FVector4f Pixel = TextureOut.GetPixel(GutterToInside.Value);
			Pixel[AttributeSettings.ClearGutterChannel] = 0.0f;
			TextureOut.SetPixel(GutterToInside.Key, Pixel);
		}
	}
	else
	{
		for (TTuple<int64, int64>& GutterToInside : OccupancyMap.GutterTexels)
		{
			TextureOut.CopyPixel(GutterToInside.Value, GutterToInside.Key);
		}
	}

	return true;
}

}} // namespace UE::PlanarCut
