// Copyright Epic Games, Inc. All Rights Reserved.
#include "GeometryMeshConversion.h"

#include "Curve/GeneralPolygon2.h"
#include "PlanarCut.h"

#include "Distance/DistLine3Segment3.h"
#include "PlanarCutPlugin.h"

#include "Distance/DistLine3Triangle3.h"
#include "MeshQueries.h"
#include "Spatial/MeshSpatialSort.h"
#include "Distance/DistSegment3Triangle3.h"
#include "VertexConnectedComponents.h"
#include "CompGeom/PolygonTriangulation.h"

#include "GeometryCollection/GeometryCollectionAlgo.h"

#include "DisjointSet.h"

#include "DynamicMeshEditor.h"
#include "DynamicMesh/DynamicMeshAABBTree3.h"
#include "Selections/MeshConnectedComponents.h"
#include "DynamicMesh/MeshTransforms.h"
#include "Operations/MeshBoolean.h"
#include "Operations/MeshSelfUnion.h"
#include "DynamicMesh/Operations/MergeCoincidentMeshEdges.h"
#include "MeshBoundaryLoops.h"
#include "QueueRemesher.h"
#include "DynamicMesh/MeshNormals.h"
#include "DynamicMesh/MeshTangents.h"
#include "ConstrainedDelaunay2.h"


#include "Algo/Rotate.h"

#if defined(_MSC_VER) && USING_CODE_ANALYSIS
#pragma warning(push)
#pragma warning(disable : 6011)
#pragma warning(disable : 6387)
#pragma warning(disable : 6313)
#pragma warning(disable : 6294)
#endif
PRAGMA_DEFAULT_VISIBILITY_START
THIRD_PARTY_INCLUDES_START
#include <Eigen/Sparse>
#include <Eigen/Core>
#include <Eigen/SparseLU>
#include <Eigen/OrderingMethods>
#include <Eigen/Dense>
THIRD_PARTY_INCLUDES_END
PRAGMA_DEFAULT_VISIBILITY_END
#if defined(_MSC_VER) && USING_CODE_ANALYSIS
#pragma warning(pop)
#endif
#include <vector> // for Eigen sparse matrix construction

using namespace UE::Geometry;


namespace UE
{
namespace PlanarCut
{

// functions to setup geometry collection attributes on dynamic meshes
namespace AugmentedDynamicMesh
{
	// An invalid color, to be replaced by neighboring valid colors (or DefaultVertexColor, if neighboring colors were not found)
	// Use a large negative value as a clear unset / invalid value, but do not go all the way to -MaxReal (or overflow will break things)
	const static FVector4f UnsetVertexColor = FVector4f(-FMathf::MaxReal * .25f, -FMathf::MaxReal * .25f, -FMathf::MaxReal * .25f, -FMathf::MaxReal * .25f);
	const static FVector4f DefaultVertexColor = FVector4f(0, 0, 0, 1);

	FName ColorAttribName = "ColorAttrib";
	FName TangentUAttribName = "TangentUAttrib";
	FName TangentVAttribName = "TangentVAttrib";
	FName VisibleAttribName = "VisibleAttrib";
	FName InternalAttribName = "InternalAttrib";
	enum
	{
		MAX_NUM_UV_CHANNELS = 8,
	};
	FName UVChannelNames[MAX_NUM_UV_CHANNELS] = {
		"UVAttrib0",
		"UVAttrib1",
		"UVAttrib2",
		"UVAttrib3",
		"UVAttrib4",
		"UVAttrib5",
		"UVAttrib6",
		"UVAttrib7"
	};

	void EnableUVChannels(FDynamicMesh3& Mesh, int32 NumUVChannels, bool bResetExisting = false, bool bDisablePrevious = true)
	{
		Mesh.EnableAttributes();
		if (!ensure(NumUVChannels <= MAX_NUM_UV_CHANNELS))
		{
			NumUVChannels = MAX_NUM_UV_CHANNELS;
		}
		for (int32 UVIdx = 0; UVIdx < NumUVChannels; UVIdx++)
		{
			if (!bResetExisting && Mesh.Attributes()->HasAttachedAttribute(UVChannelNames[UVIdx]))
			{
				continue;
			}
			Mesh.Attributes()->AttachAttribute(UVChannelNames[UVIdx], new TDynamicMeshVertexAttribute<float, 2>(&Mesh));
		}
		if (bDisablePrevious)
		{
			for (int32 UVIdx = NumUVChannels; UVIdx < MAX_NUM_UV_CHANNELS; UVIdx++)
			{
				Mesh.Attributes()->RemoveAttribute(UVChannelNames[UVIdx]);
			}
		}
	}

	int32 NumEnabledUVChannels(FDynamicMesh3& Mesh)
	{
		if (!Mesh.Attributes())
		{
			return 0;
		}
		for (int32 UVIdx = 0; UVIdx < MAX_NUM_UV_CHANNELS; UVIdx++)
		{
			if (!Mesh.Attributes()->HasAttachedAttribute(UVChannelNames[UVIdx]))
			{
				return UVIdx;
			}
		}
		return MAX_NUM_UV_CHANNELS;
	}

	void Augment(FDynamicMesh3& Mesh, int32 NumUVChannels)
	{
		Mesh.EnableVertexNormals(FVector3f::UnitZ());
		Mesh.EnableAttributes();
		Mesh.Attributes()->EnableMaterialID();
		Mesh.Attributes()->AttachAttribute(ColorAttribName, new TDynamicMeshVertexAttribute<float, 4>(&Mesh));
		Mesh.Attributes()->AttachAttribute(TangentUAttribName, new TDynamicMeshVertexAttribute<float, 3>(&Mesh));
		Mesh.Attributes()->AttachAttribute(TangentVAttribName, new TDynamicMeshVertexAttribute<float, 3>(&Mesh));
		TDynamicMeshScalarTriangleAttribute<bool>* VisAttrib = new TDynamicMeshScalarTriangleAttribute<bool>(&Mesh);
		VisAttrib->Initialize(true);
		Mesh.Attributes()->AttachAttribute(VisibleAttribName, VisAttrib);
		TDynamicMeshScalarTriangleAttribute<bool>* InternalAttrib = new TDynamicMeshScalarTriangleAttribute<bool>(&Mesh);
		InternalAttrib->Initialize(true);
		Mesh.Attributes()->AttachAttribute(InternalAttribName, InternalAttrib);

		EnableUVChannels(Mesh, NumUVChannels);
	}

	void AddVertexColorAttribute(FDynamicMesh3& Mesh)
	{
		Mesh.Attributes()->AttachAttribute(ColorAttribName, new TDynamicMeshVertexAttribute<float, 4>(&Mesh));
	}

	bool IsAugmented(const FDynamicMesh3& Mesh)
	{
		return Mesh.HasAttributes()
			&& Mesh.Attributes()->HasAttachedAttribute(ColorAttribName)
			&& Mesh.Attributes()->HasAttachedAttribute(TangentUAttribName)
			&& Mesh.Attributes()->HasAttachedAttribute(TangentVAttribName)
			&& Mesh.Attributes()->HasAttachedAttribute(VisibleAttribName)
			&& Mesh.Attributes()->HasMaterialID()
			&& Mesh.HasVertexNormals();
	}

	void SetDefaultAttributes(FDynamicMesh3& Mesh, bool bGlobalVisibility)
	{
		checkSlow(IsAugmented(Mesh));
		TDynamicMeshVertexAttribute<float, 4>* Colors =
			static_cast<TDynamicMeshVertexAttribute<float, 4>*>(Mesh.Attributes()->GetAttachedAttribute(ColorAttribName));
		TDynamicMeshVertexAttribute<float, 3>* Us =
			static_cast<TDynamicMeshVertexAttribute<float, 3>*>(Mesh.Attributes()->GetAttachedAttribute(TangentUAttribName));
		TDynamicMeshVertexAttribute<float, 3>* Vs =
			static_cast<TDynamicMeshVertexAttribute<float, 3>*>(Mesh.Attributes()->GetAttachedAttribute(TangentVAttribName));

		for (int VID : Mesh.VertexIndicesItr())
		{
			FVector3f N = Mesh.GetVertexNormal(VID);
			FVector3f U, V;
			VectorUtil::MakePerpVectors(N, U, V);
			Us->SetValue(VID, U);
			Vs->SetValue(VID, V);
			Colors->SetValue(VID, UnsetVertexColor);
		}

		TDynamicMeshScalarTriangleAttribute<bool>* Visible =
			static_cast<TDynamicMeshScalarTriangleAttribute<bool>*>(Mesh.Attributes()->GetAttachedAttribute(VisibleAttribName));
		for (int TID : Mesh.TriangleIndicesItr())
		{
			Visible->SetNewValue(TID, bGlobalVisibility);
		}
	}

	void SetVisibility(FDynamicMesh3& Mesh, int TID, bool bIsVisible)
	{
		checkSlow(IsAugmented(Mesh));
		TDynamicMeshScalarTriangleAttribute<bool>* Visible =
			static_cast<TDynamicMeshScalarTriangleAttribute<bool>*>(Mesh.Attributes()->GetAttachedAttribute(VisibleAttribName));
		Visible->SetValue(TID, bIsVisible);
	}

	bool GetVisibility(const FDynamicMesh3& Mesh, int TID)
	{
		checkSlow(IsAugmented(Mesh));
		const TDynamicMeshScalarTriangleAttribute<bool>* Visible =
			static_cast<const TDynamicMeshScalarTriangleAttribute<bool>*>(Mesh.Attributes()->GetAttachedAttribute(VisibleAttribName));
		return Visible->GetValue(TID);
	}

	void SetInternal(FDynamicMesh3& Mesh, int TID, bool bIsInternal)
	{
		checkSlow(IsAugmented(Mesh));
		TDynamicMeshScalarTriangleAttribute<bool>* Internal = 
			static_cast<TDynamicMeshScalarTriangleAttribute<bool>*>(Mesh.Attributes()->GetAttachedAttribute(InternalAttribName));
		Internal->SetValue(TID, bIsInternal);
	}

	bool GetInternal(const FDynamicMesh3& Mesh, int TID)
	{
		checkSlow(IsAugmented(Mesh));
		const TDynamicMeshScalarTriangleAttribute<bool>* Internal =
			static_cast<const TDynamicMeshScalarTriangleAttribute<bool>*>(Mesh.Attributes()->GetAttachedAttribute(InternalAttribName));
		return Internal->GetValue(TID);
	}

	void SetUV(FDynamicMesh3& Mesh, int VID, FVector2f UV, int UVLayer)
	{
		if (!ensure(UVLayer < MAX_NUM_UV_CHANNELS))
		{
			return;
		}
		TDynamicMeshVertexAttribute<float, 2>* UVs =
			static_cast<TDynamicMeshVertexAttribute<float, 2>*>(Mesh.Attributes()->GetAttachedAttribute(UVChannelNames[UVLayer]));
		if (ensure(UVs))
		{
			UVs->SetValue(VID, UV);
		}
	}

	void SetAllUV(FDynamicMesh3& Mesh, int VID, FVector2f UV, int NumUVLayers)
	{
		if (!ensure(NumUVLayers <= MAX_NUM_UV_CHANNELS))
		{
			NumUVLayers = MAX_NUM_UV_CHANNELS;
		}
		for (int32 Layer = 0; Layer < NumUVLayers; Layer++)
		{
			TDynamicMeshVertexAttribute<float, 2>* UVs =
				static_cast<TDynamicMeshVertexAttribute<float, 2>*>(Mesh.Attributes()->GetAttachedAttribute(UVChannelNames[Layer]));
			if (ensure(UVs))
			{
				UVs->SetValue(VID, UV);
			}
		}
	}

	void GetUV(const FDynamicMesh3& Mesh, int VID, FVector2f& UV, int UVLayer)
	{
		if (!ensure(UVLayer < MAX_NUM_UV_CHANNELS))
		{
			UV = FVector2f::Zero();
			return;
		}
		const TDynamicMeshVertexAttribute<float, 2>* UVs =
			static_cast<const TDynamicMeshVertexAttribute<float, 2>*>(Mesh.Attributes()->GetAttachedAttribute(UVChannelNames[UVLayer]));
		if (ensure(UVs))
		{
			UVs->GetValue(VID, UV);
		}
	}

	void SetTangent(FDynamicMesh3& Mesh, int VID, FVector3f Normal, FVector3f TangentU, FVector3f TangentV)
	{
		checkSlow(IsAugmented(Mesh));
		TDynamicMeshVertexAttribute<float, 3>* Us =
			static_cast<TDynamicMeshVertexAttribute<float, 3>*>(Mesh.Attributes()->GetAttachedAttribute(TangentUAttribName));
		TDynamicMeshVertexAttribute<float, 3>* Vs =
			static_cast<TDynamicMeshVertexAttribute<float, 3>*>(Mesh.Attributes()->GetAttachedAttribute(TangentVAttribName));
		Us->SetValue(VID, TangentU);
		Vs->SetValue(VID, TangentV);
	}

	void GetTangent(const FDynamicMesh3& Mesh, int VID, FVector3f& U, FVector3f& V)
	{
		checkSlow(IsAugmented(Mesh));
		const TDynamicMeshVertexAttribute<float, 3>* Us =
			static_cast<const TDynamicMeshVertexAttribute<float, 3>*>(Mesh.Attributes()->GetAttachedAttribute(TangentUAttribName));
		const TDynamicMeshVertexAttribute<float, 3>* Vs =
			static_cast<const TDynamicMeshVertexAttribute<float, 3>*>(Mesh.Attributes()->GetAttachedAttribute(TangentVAttribName));
		FVector3f Normal = Mesh.GetVertexNormal(VID);
		Us->GetValue(VID, U);
		Vs->GetValue(VID, V);
	}

	FVector4f GetVertexColor(const FDynamicMesh3& Mesh, int VID)
	{
		checkSlow(IsAugmented(Mesh));
		const TDynamicMeshVertexAttribute<float, 4>* Colors =
			static_cast<const TDynamicMeshVertexAttribute<float, 4>*>(Mesh.Attributes()->GetAttachedAttribute(ColorAttribName));
		FVector4f Color;
		Colors->GetValue(VID, Color);
		return Color;
	}

	void SetVertexColor(FDynamicMesh3& Mesh, int VID, FVector4f Color)
	{
		checkSlow(IsAugmented(Mesh));
		TDynamicMeshVertexAttribute<float, 4>* Colors =
			static_cast<TDynamicMeshVertexAttribute<float, 4>*>(Mesh.Attributes()->GetAttachedAttribute(ColorAttribName));
		Colors->SetValue(VID, Color);
	}

	template<typename FOverlay, typename VecType>
	void SplitOverlayHelper(UE::Geometry::FDynamicMesh3& Mesh, FOverlay* Overlay)
	{
		TArray<int32> ContigTris, ContigGroupLens;
		TArray<int32> ToSplitTris;
		TArray<bool> GroupIsLoop;

		auto GetEID = [&Mesh, &Overlay](int32 VID, int32 TID) -> int32
		{
			FIndex3i Tri = Mesh.GetTriangle(TID);
			int32 SubIdx = Tri.IndexOf(VID);
			check(SubIdx >= 0);
			return Overlay->GetTriangle(TID)[SubIdx];
		};

		// get overlay element, returning a zero vector for unset elements
		auto GetElement = [&Overlay](int32 EID)
		{
			return EID > -1 ? Overlay->GetElement(EID) : VecType(EForceInit::ForceInitToZero);
		};

		for (int32 VID = 0, OrigMax = Mesh.MaxVertexID(); VID < OrigMax; VID++)
		{
			if (!Mesh.IsVertex(VID))
			{
				continue;
			}
			Mesh.GetVtxContiguousTriangles(VID, ContigTris, ContigGroupLens, GroupIsLoop);
			int32 TriStartIdx = 0;
			for (int32 GroupIdx = 0; GroupIdx < ContigGroupLens.Num(); GroupIdx++)
			{
				int32 GroupLen = ContigGroupLens[GroupIdx];
				bool bIsLoop = GroupIsLoop[GroupIdx];
				checkSlow(TriStartIdx < ContigTris.Num());
				int32 TID0 = ContigTris[TriStartIdx];
				int32 EID0 = GetEID(VID, TID0);
				VecType LastElData = GetElement(EID0);
				int32 LastEID = EID0;
				bool bPastEID0 = false;
				ToSplitTris.Reset();
				for (int32 Idx = TriStartIdx + 1, EndIdx = TriStartIdx + GroupLen; Idx < EndIdx; Idx++)
				{
					int32 TID = ContigTris[Idx];
					int32 EID = GetEID(VID, TID);
					VecType ElData;
					if (EID != LastEID &&
						!VectorUtil::EpsilonEqual(ElData = GetElement(EID), LastElData, FMathf::ZeroTolerance))
					{
						if (ToSplitTris.Num() > 0)
						{
							FDynamicMesh3::FVertexSplitInfo SplitInfo;
							Mesh.SplitVertex(VID, ToSplitTris, SplitInfo);
							ToSplitTris.Reset();
						}
						LastEID = EID;
						LastElData = ElData;
						bPastEID0 = true;
					}
					if (bPastEID0)
					{
						ToSplitTris.Add(TID);
					}
				}
				if (bPastEID0 && ToSplitTris.Num() > 0 && (!bIsLoop || LastEID != EID0))
				{
					// one final group at the end
					FDynamicMesh3::FVertexSplitInfo SplitInfo;
					Mesh.SplitVertex(VID, ToSplitTris, SplitInfo);
				}
				TriStartIdx += GroupLen;
			}
		}
	}

	void SplitOverlayAttributesToPerVertex(UE::Geometry::FDynamicMesh3& Mesh, bool bSplitUVs, bool bSplitNormalsTangents)
	{
		if (!ensure(Mesh.HasAttributes()))
		{
			return;
		}

		int32 NumUVLayers = FMath::Min(NumEnabledUVChannels(Mesh), Mesh.Attributes()->NumUVLayers());
		if (bSplitUVs)
		{
			for (int32 LayerIdx = 0; LayerIdx < NumUVLayers; LayerIdx++)
			{
				FDynamicMeshUVOverlay* UVOverlay = Mesh.Attributes()->GetUVLayer(LayerIdx);
				SplitOverlayHelper<FDynamicMeshUVOverlay, FVector2f>(Mesh, UVOverlay);
			}
		}

		if (bSplitNormalsTangents)
		{
			for (int32 LayerIdx = 0; LayerIdx < Mesh.Attributes()->NumNormalLayers(); LayerIdx++)
			{
				SplitOverlayHelper<FDynamicMeshNormalOverlay, FVector3f>(Mesh, Mesh.Attributes()->GetNormalLayer(LayerIdx));
			}
		}

		FDynamicMeshEditor Editor(&Mesh);
		FDynamicMeshEditResult EditResult;
		Editor.SplitBowties(EditResult);

		// copy back attributes
		for (int32 LayerIdx = 0; LayerIdx < NumUVLayers; LayerIdx++)
		{
			FDynamicMeshUVOverlay* UVOverlay = Mesh.Attributes()->GetUVLayer(LayerIdx);
			TDynamicMeshVertexAttribute<float, 2>* UVs =
				static_cast<TDynamicMeshVertexAttribute<float, 2>*>(Mesh.Attributes()->GetAttachedAttribute(UVChannelNames[LayerIdx]));
			for (int32 EID : UVOverlay->ElementIndicesItr())
			{
				int32 ParentVID = UVOverlay->GetParentVertex(EID);
				if (ParentVID != INDEX_NONE)
				{
					UVs->SetValue(ParentVID, UVOverlay->GetElement(EID));
				}
			}
		}
		if (Mesh.Attributes()->NumNormalLayers() > 0)
		{
			FDynamicMeshNormalOverlay* Overlay = Mesh.Attributes()->GetNormalLayer(0);
			for (int32 EID : Overlay->ElementIndicesItr())
			{
				int32 ParentVID = Overlay->GetParentVertex(EID);
				if (ParentVID != INDEX_NONE)
				{
					Mesh.SetVertexNormal(ParentVID, Overlay->GetElement(EID));
				}
			}
		}
		if (Mesh.Attributes()->HasTangentSpace())
		{
			TDynamicMeshVertexAttribute<float, 3>* Us =
				static_cast<TDynamicMeshVertexAttribute<float, 3>*>(Mesh.Attributes()->GetAttachedAttribute(TangentUAttribName));
			TDynamicMeshVertexAttribute<float, 3>* Vs =
				static_cast<TDynamicMeshVertexAttribute<float, 3>*>(Mesh.Attributes()->GetAttachedAttribute(TangentVAttribName));
			TDynamicMeshVertexAttribute<float, 3>* Tangents[] = { Us, Vs };
			for (int Idx = 1; Idx < 3; Idx++)
			{
				FDynamicMeshNormalOverlay* Overlay = Mesh.Attributes()->GetNormalLayer(Idx);
				TDynamicMeshVertexAttribute<float, 3>* Tangent = Tangents[Idx - 1];
				for (int32 EID : Overlay->ElementIndicesItr())
				{
					int32 ParentVID = Overlay->GetParentVertex(EID);
					if (ParentVID != INDEX_NONE)
					{
						Tangent->SetValue(ParentVID, Overlay->GetElement(EID));
					}
				}
			}
		}
	}
	
	void InitializeOverlayToPerVertexColors(FDynamicMesh3& Mesh)
	{
		if (!ensure(Mesh.HasAttributes()))
		{
			return;
		}
		Mesh.Attributes()->EnablePrimaryColors();
		Mesh.Attributes()->PrimaryColors()->CreatePerVertex(0.f);
		Mesh.EnableVertexColors(FVector3f::Zero());

		if(Mesh.Attributes())
		{
			FDynamicMeshColorOverlay* const ColorOverlay = Mesh.Attributes()->PrimaryColors();
			for (const int TriangleID : Mesh.TriangleIndicesItr())
			{
				const FIndex3i Tri = Mesh.GetTriangle(TriangleID);
				const FIndex3i ColorElementTri = ColorOverlay->GetTriangle(TriangleID);
				for (int TriVertIndex = 0; TriVertIndex < 3; ++TriVertIndex)
				{
					ColorOverlay->SetElement(ColorElementTri[TriVertIndex],
						AugmentedDynamicMesh::GetVertexColor(Mesh, Tri[TriVertIndex]));
				}
			}
		}
	}

	void InitializeOverlayToPerVertexUVs(FDynamicMesh3& Mesh, int32 NumUVLayers, int32 FirstUVLayer)
	{
		Mesh.Attributes()->SetNumUVLayers(NumUVLayers);
		for (int UVLayer = 0; UVLayer < NumUVLayers; ++UVLayer)
		{
			FDynamicMeshUVOverlay* UVs = Mesh.Attributes()->GetUVLayer(UVLayer);
			UVs->ClearElements();
			TArray<int> VertToUVMap;
			VertToUVMap.SetNumUninitialized(Mesh.MaxVertexID());
			for (int VID : Mesh.VertexIndicesItr())
			{
				FVector2f UV;
				GetUV(Mesh, VID, UV, UVLayer + FirstUVLayer);
				int UVID = UVs->AppendElement(UV);
				VertToUVMap[VID] = UVID;
			}
			for (int TID : Mesh.TriangleIndicesItr())
			{
				FIndex3i Tri = Mesh.GetTriangle(TID);
				Tri.A = VertToUVMap[Tri.A];
				Tri.B = VertToUVMap[Tri.B];
				Tri.C = VertToUVMap[Tri.C];
				UVs->SetTriangle(TID, Tri);
			}
		}
	}

	void InitializeOverlayToPerVertexTangents(FDynamicMesh3& Mesh)
	{
		Mesh.Attributes()->EnableTangents();
		FDynamicMeshNormalOverlay* TangentOverlays[2] = { Mesh.Attributes()->PrimaryTangents(), Mesh.Attributes()->PrimaryBiTangents() };
		TangentOverlays[0]->ClearElements();
		TangentOverlays[1]->ClearElements();
		TArray<int> VertToTangentMap;
		VertToTangentMap.SetNumUninitialized(Mesh.MaxVertexID());
		for (int VID : Mesh.VertexIndicesItr())
		{
			FVector3f Tangents[2];
			GetTangent(Mesh, VID, Tangents[0], Tangents[1]);
			int TID = TangentOverlays[0]->AppendElement(&Tangents[0].X);
			int TID2 = TangentOverlays[1]->AppendElement(&Tangents[1].X);
			check(TID == TID2);
			VertToTangentMap[VID] = TID;
		}
		for (int TID : Mesh.TriangleIndicesItr())
		{
			FIndex3i Tri = Mesh.GetTriangle(TID);
			Tri.A = VertToTangentMap[Tri.A];
			Tri.B = VertToTangentMap[Tri.B];
			Tri.C = VertToTangentMap[Tri.C];
			TangentOverlays[0]->SetTriangle(TID, Tri);
			TangentOverlays[1]->SetTriangle(TID, Tri);
		}
	}

	void ComputeTangents(FDynamicMesh3& Mesh, bool bOnlyInternalSurfaces, 
		bool bRecomputeNormals, bool bMakeSharpEdges, float SharpAngleDegrees)
	{
		bMakeSharpEdges = bMakeSharpEdges && bRecomputeNormals; // cannot make sharp edges if normals aren't supposed to change

		FDynamicMeshNormalOverlay* Normals = Mesh.Attributes()->PrimaryNormals();
		FMeshNormals::InitializeOverlayToPerVertexNormals(Normals, !bRecomputeNormals || bMakeSharpEdges);

		// Copy per-vertex UVs to a UV overlay, because that's what the tangents code uses
		// (TODO: consider making a tangent computation path that uses vertex normals / UVs)
		// only need 1 UV layer unless we're round-tripping all the attributes through the mesh
		int32 NeedNumUVLayers = bMakeSharpEdges ? NumEnabledUVChannels(Mesh) : 1;
		InitializeOverlayToPerVertexUVs(Mesh, NeedNumUVLayers, 0);
		FDynamicMeshUVOverlay* UVs = Mesh.Attributes()->PrimaryUV();
		TDynamicMeshScalarTriangleAttribute<bool>* InternalAttrib =
			static_cast<TDynamicMeshScalarTriangleAttribute<bool>*>(Mesh.Attributes()->GetAttachedAttribute(AugmentedDynamicMesh::InternalAttribName));

		auto ShouldUpdateInternal = [&InternalAttrib, bOnlyInternalSurfaces](int TID)
		{
			return !bOnlyInternalSurfaces || InternalAttrib->GetValue(TID);
		};

		// To update the normals topology, we need to weld and re-split the whole mesh
		if (bMakeSharpEdges)
		{
			FDynamicMeshNormalOverlay NormalsOverlayCopy(&Mesh);

			// Apply coincident edge merge so that sharp normals can be smoothed where the edge angle is below the threshold
			AugmentedDynamicMesh::InitializeOverlayToPerVertexTangents(Mesh);
			FMergeCoincidentMeshEdges EdgeWelder(&Mesh);
			EdgeWelder.Apply();

			NormalsOverlayCopy.Copy(*Normals);
			// need to re-topo where the materials match WhichMaterials
			double NormalDotProdThreshold = FMathd::Cos(SharpAngleDegrees * FMathd::DegToRad);

			FMeshNormals FaceNormals(&Mesh);
			FaceNormals.ComputeTriangleNormals();
			Normals->CreateFromPredicate([&](int VID, int TA, int TB) {
				bool IA = InternalAttrib->GetValue(TA), IB = InternalAttrib->GetValue(TB);
				if (IA != IB)
				{
					return false; // always split at an internal/external face boundary
				}
				bool bShouldUpdateTopo = !bOnlyInternalSurfaces || (IA && IB);
				if (bShouldUpdateTopo) // in the region we're updating, don't split above dot threshold
				{
					return FaceNormals[TA].Dot(FaceNormals[TB]) > NormalDotProdThreshold;
				}
				else // in the region we're not updating, keep the original connectivity
				{
					return NormalsOverlayCopy.AreTrianglesConnected(TA, TB);
				}
			}, 0);

			// copy back recomputed normals to the triangles that need updating, original normals to the triangles we don't want to change
			FMeshNormals RecomputedNormals(&Mesh);
			RecomputedNormals.RecomputeOverlayNormals(Normals);
			for (int TID : Mesh.TriangleIndicesItr())
			{
				FIndex3i OverlayTri = Normals->GetTriangle(TID);
				if (ShouldUpdateInternal(TID))
				{
					for (int SubIdx = 0; SubIdx < 3; SubIdx++)
					{
						int ElementID = OverlayTri[SubIdx];
						Normals->SetElement(ElementID, (FVector3f)RecomputedNormals[ElementID]);
					}
				}
				else
				{
					FIndex3i OldOverlayTri = NormalsOverlayCopy.GetTriangle(TID);
					for (int SubIdx = 0; SubIdx < 3; SubIdx++)
					{
						int NewElementID = OverlayTri[SubIdx];
						int OldElementID = OldOverlayTri[SubIdx];
						Normals->SetElement(NewElementID, NormalsOverlayCopy.GetElement(OldElementID));
					}
				}
			}

			Mesh.CompactInPlace();

			// Re-split vertices with different UVs/normals/tangents and transfer attributes back (required because we weld edges above)
			AugmentedDynamicMesh::SplitOverlayAttributesToPerVertex(Mesh, true, true);
		}

		FComputeTangentsOptions Options;
		Options.bAngleWeighted = true;
		Options.bAveraged = true;
		FMeshTangentsf Tangents(&Mesh);
		Tangents.ComputeTriVertexTangents(Normals, UVs, Options);

		const TArray<FVector3f>& TanU = Tangents.GetTangents();
		const TArray<FVector3f>& TanV = Tangents.GetBitangents();
		for (int TID : Mesh.TriangleIndicesItr())
		{
			if (!ShouldUpdateInternal(TID))
			{
				continue;
			}
					
			int TanIdxBase = TID * 3;
			FIndex3i Tri = Mesh.GetTriangle(TID);
			
			FIndex3i NormalElTri;
			if (!bMakeSharpEdges) // if sharp edges, normals were already copied to the vertices, above
			{
				NormalElTri = Normals->GetTriangle(TID);
			}
			for (int Idx = 0; Idx < 3; Idx++)
			{
				int VID = Tri[Idx];
				int TanIdx = TanIdxBase + Idx;
				FVector3f VertexNormal = bMakeSharpEdges ? Mesh.GetVertexNormal(VID) : Normals->GetElement(NormalElTri[Idx]);
				if (!bMakeSharpEdges && bRecomputeNormals)
				{
					Mesh.SetVertexNormal(VID, VertexNormal);
				}
				SetTangent(Mesh, VID, VertexNormal, TanU[TanIdx], TanV[TanIdx]);
			}
		}
	}

	// per component sampling is a rough heuristic to avoid doing geodesic distance but still get points on a 'thin' slice
	void AddCollisionSamplesPerComponent(FDynamicMesh3& Mesh, double Spacing)
	{
		checkSlow(IsAugmented(Mesh));

		// We use 0 or negative spacing to disable collision samples
		if (Spacing <= 0)
		{
			return;
		}

		// Set an approx ceiling for max collision samples; these are only used for point-implicit collision so 10k should be well above the max useful
		constexpr int32 MaxCollisionSampleCount = 10000;
		double UseSpacing = Spacing;
		const double TotalArea = TMeshQueries<FDynamicMesh3>::GetVolumeArea(Mesh).Y;
		const double MinSpacing = FMathd::Sqrt(2.0 * TotalArea / double(MaxCollisionSampleCount));
		if (MinSpacing > Spacing)
		{
			UseSpacing = MinSpacing;
			UE_LOG(LogPlanarCut, Warning,
				TEXT("Collision sample spacing of %g is too small for this geometry (would generate an excessive number of vertices); clamping to %g."),
				Spacing, UseSpacing);
		}

		FMeshConnectedComponents Components(&Mesh);
		// TODO: if/when we switch to merged edges representation, pass a predicate here based on whether there's a normal seam ?
		Components.FindConnectedTriangles();
		TArray<TPointHashGrid3d<int>> KnownSamples;  KnownSamples.Reserve(Components.Num());
		for (int ComponentIdx = 0; ComponentIdx < Components.Num(); ComponentIdx++)
		{
			KnownSamples.Emplace(.5 * UseSpacing / FMathd::InvSqrt3, -1);
		}

		TArray<int> AlreadySeen; AlreadySeen.Init(-1, Mesh.MaxVertexID());
		for (int ComponentIdx = 0; ComponentIdx < Components.Num(); ComponentIdx++)
		{
			FMeshConnectedComponents::FComponent& Component = Components.GetComponent(ComponentIdx);
			for (int TID : Component.Indices)
			{
				FIndex3i Tri = Mesh.GetTriangle(TID);
				for (int SubIdx = 0; SubIdx < 3; SubIdx++)
				{
					int VID = Tri[SubIdx];
					if (AlreadySeen[VID] != ComponentIdx)
					{
						AlreadySeen[VID] = ComponentIdx;
						KnownSamples[ComponentIdx].InsertPointUnsafe(VID, Mesh.GetVertex(VID));
					}
				}
			}
		}
		AlreadySeen.Empty();

		double SpacingThreshSq = UseSpacing * UseSpacing; // if points are more than Spacing apart, consider adding a new point between them
		for (int ComponentIdx = 0; ComponentIdx < Components.Num(); ComponentIdx++)
		{
			FMeshConnectedComponents::FComponent& Component = Components.GetComponent(ComponentIdx);
			for (int TID : Component.Indices)
			{
				FIndex3i TriVIDs = Mesh.GetTriangle(TID);
				FTriangle3d Triangle;
				Mesh.GetTriVertices(TID, Triangle.V[0], Triangle.V[1], Triangle.V[2]);
				double EdgeLensSq[3];
				int MaxEdgeIdx = 0;
				double MaxEdgeLenSq = 0;
				for (int i = 2, j = 0; j < 3; i = j++)
				{
					double EdgeLenSq = DistanceSquared(Triangle.V[i], Triangle.V[j]);
					if (EdgeLenSq > MaxEdgeLenSq)
					{
						MaxEdgeIdx = i;
						MaxEdgeLenSq = EdgeLenSq;
					}
					EdgeLensSq[i] = EdgeLenSq;
				}
				// if we found a too-long edge, we can try sampling the tri
				if (MaxEdgeLenSq > SpacingThreshSq)
				{
					FVector3f Normal = (FVector3f)VectorUtil::Normal(Triangle.V[0], Triangle.V[1], Triangle.V[2]);

					// Pick number of samples based on the longest edge
					double LongEdgeLen = FMathd::Sqrt(MaxEdgeLenSq);
					int Divisions = FMath::FloorToInt32(LongEdgeLen / UseSpacing);
					double Factor = 1.0 / double(Divisions + 1);
					int SecondEdgeIdx = (MaxEdgeIdx + 1) % 3;
					int ThirdEdgeIdx = (MaxEdgeIdx + 2) % 3;
					// Sample along the two longest edges first, then interpolate these samples
					int SecondLongestEdgeIdx = SecondEdgeIdx;
					if (EdgeLensSq[SecondEdgeIdx] < EdgeLensSq[ThirdEdgeIdx])
					{
						SecondLongestEdgeIdx = ThirdEdgeIdx;
					}
					int SecondLongestSecondEdgeIdx = (SecondLongestEdgeIdx + 1) % 3;
					for (int DivI = 0; DivI < Divisions; DivI++)
					{
						double Along = (DivI + 1) * Factor;
						FVector3d E1Bary(0, 0, 0), E2Bary(0, 0, 0);
						E1Bary[MaxEdgeIdx] = Along;
						E1Bary[SecondEdgeIdx] = 1 - Along;
						E2Bary[SecondLongestEdgeIdx] = 1 - Along;
						E2Bary[SecondLongestSecondEdgeIdx] = Along;

						// Choose number of samples between the two edge points based on their distance
						double AcrossDist = Distance( Triangle.BarycentricPoint(E1Bary), Triangle.BarycentricPoint(E2Bary) );
						int DivisionsAcross = FMath::CeilToInt32(AcrossDist / UseSpacing);
						double FactorAcross = 1.0 / double(DivisionsAcross + 1);
						for (int DivJ = 0; DivJ < DivisionsAcross; DivJ++)
						{
							double AlongAcross = (DivJ + 1) * FactorAcross;
							FVector3d Bary = UE::Geometry::Lerp(E1Bary, E2Bary, AlongAcross);
							FVector3d SamplePos = Triangle.BarycentricPoint(Bary);
							if (!KnownSamples[ComponentIdx].IsCellEmptyUnsafe(SamplePos)) // fast early out; def. have pt within radius
							{
								continue;
							}
							TPair<int, double> VIDDist = KnownSamples[ComponentIdx].FindNearestInRadius(SamplePos, UseSpacing * .5, [&Mesh, SamplePos](int VID)
								{
									return DistanceSquared(Mesh.GetVertex(VID), SamplePos);
								});
							// No point within radius Spacing/2 -> Add a new sample
							if (VIDDist.Key == -1)
							{
								// no point within radius; can add a sample here
								FVertexInfo Info(SamplePos, Normal);

								int AddedVID = Mesh.AppendVertex(Info);
								SetVertexColor(Mesh, AddedVID, DefaultVertexColor);
								KnownSamples[ComponentIdx].InsertPointUnsafe(AddedVID, SamplePos);
							}
						}
					}
				}
			}
		}
	}

	/**
	 * Fill holes in non-welded meshes, w/ UVs that attempt to continue one of the local UV islands.
	 * Intended to clean holes erroneously left by FMeshBoolean (due to floating point error)
	 * 
	 * Note this algo tries to continue UVs of internal faces only, where the valid UV areas aren't already
	 * pre-defined to match an artist-authored texture, but are automatically generated by simple projection
	 * per connected component.
	 * Hole filling with 'external' materials and UVs could risk having very stretched triangles in UV space.
	 * The hope is that holes are rare and small enough that it's ok to just use internal materials.
	 * 
	 * @param Mesh Fill holes on this mesh
	 * @param CandidateEdges Only consider holes that touch these edges
	 */
	void FillHoles(FDynamicMesh3& Mesh, const TSet<int32>& CandidateEdges, double MinHoleArea = DOUBLE_KINDA_SMALL_NUMBER)
	{
		/// 1. Build a spatial hash of boundary vertices, so we can identify holes even though the mesh is not welded
		
		double SnapDistance = 1e-03;
		TPointHashGrid3d<int32> VertHash(SnapDistance * 10, -1);
		TMap<int32, int32> CoincidentVerticesMap; // map every vertex in an overlapping cluster to a canonical single vertex for that cluster
		TSet<int32> HashedVertices; // track what's already processed

		auto AddVertexToHash = [&Mesh, SnapDistance, &VertHash, &CoincidentVerticesMap, &HashedVertices](int32 VID)
		{
			if (HashedVertices.Contains(VID))
			{
				return;
			}

			HashedVertices.Add(VID);

			FVector3d VPos = Mesh.GetVertex(VID);
			TPair<int32, double> Res = VertHash.FindNearestInRadius(VPos, SnapDistance, [&Mesh, &VPos](const int& VID)->double { return FVector3d::DistSquared(VPos, Mesh.GetVertex(VID)); });
			if (Res.Key != INDEX_NONE)
			{
				int32 MapTo = Res.Key;
				int32* WasMapped = CoincidentVerticesMap.Find(MapTo);
				if (WasMapped)
				{
					MapTo = *WasMapped;
				}
				CoincidentVerticesMap.Add(VID, MapTo);
			}

			VertHash.InsertPointUnsafe(VID, VPos);
		};

		for (int32 EID : CandidateEdges)
		{
			FDynamicMesh3::FEdge Edge = Mesh.GetEdge(EID);
			if (Edge.Tri.B == -1) // only consider boundary edges
			{
				AddVertexToHash(Edge.Vert.A);
				AddVertexToHash(Edge.Vert.B);
			}
		}

		/// 2. Find the edges w/ no paired reverse edge (i.e. the potential hole boundary edges)

		using FPair = TPair<int32, int32>;
		TMultiMap<int32, FPair> OpenEdges; // FPair contains: (Second vertex ID on edge, Edge ID)
		auto CanonicalVID = [&CoincidentVerticesMap](int32 VID)
		{
			int32* Mapped = CoincidentVerticesMap.Find(VID);
			return Mapped ? *Mapped : VID;
		};
		// remove an edge from OpenEdges, assuming the vertices are already passed through CanonicalVID()
		auto RemoveCanonEdge = [&OpenEdges](int32 A, int32 B)
		{
			int32 NumRemovedPairs = 0;
			for (TMultiMap<int32, FPair>::TKeyIterator It = OpenEdges.CreateKeyIterator(A); It; ++It)
			{
				if (It.Value().Key == B)
				{
					It.RemoveCurrent();
					return true;
				}
			}
			return false;
		};
		auto AddEdge = [&OpenEdges, &CanonicalVID, &RemoveCanonEdge](int32 A, int32 B, int32 EID)
		{
			A = CanonicalVID(A);
			B = CanonicalVID(B);
			if (A == B) // skip edges where the vertices were too close
			{
				return;
			}
			if (!RemoveCanonEdge(B, A))
			{
				OpenEdges.Add(A, FPair(B, EID));
			}
		};
		for (int32 CandEID : CandidateEdges)
		{
			FDynamicMesh3::FEdge Edge = Mesh.GetEdge(CandEID);
			if (Edge.Tri[1] == FDynamicMesh3::InvalidID)
			{
				int32 TID = Edge.Tri[0];
				FIndex2i EdgeV = Mesh.GetOrientedBoundaryEdgeV(CandEID);
				AddEdge(EdgeV.A, EdgeV.B, CandEID);
			}
		}

		if (OpenEdges.Num() < 3)
		{
			return;
		}

		/// 3. Walk the open boundary edges to find the loops that define holes

		TArray<int32> AllBoundaryVerts; // VIDs for all hole boundaries (w/ Num IDs at the start of each span)
		TArray<int32> AllBoundaryEdges; // EIDs for hole boundaries (w/ same layout as AllBoundaryVerts)
		int32 NumHoles = 0;
		while (true)
		{
			const auto& EdgeItr = OpenEdges.CreateConstIterator();
			if (!EdgeItr)
			{
				break;
			}
			int32 Start = EdgeItr.Key();
			FPair WalkPair = EdgeItr.Value();
			int32 Walk = WalkPair.Key;
			int32 BdryStartIdx = AllBoundaryVerts.Add(-1);
			int32 BdryEdgeStartIdx = AllBoundaryEdges.Add(-1);
			checkSlow(BdryEdgeStartIdx == BdryStartIdx);
			AllBoundaryVerts.Add(Start);
			AllBoundaryEdges.Add(WalkPair.Value);
			RemoveCanonEdge(Start, Walk);
			while (Walk != Start) // this will either loop back to start or dead end (note every non-dead-end iter removes an edge from OpenEdges)
			{
				AllBoundaryVerts.Add(Walk);
				FPair* Next = OpenEdges.Find(Walk);
				if (!Next)
				{
					// dead-ended chain of edges, discard the vertices
					AllBoundaryVerts.SetNum(BdryStartIdx);
					AllBoundaryEdges.SetNum(BdryStartIdx);
					break;
				}
				int32 NextVal = Next->Key;
				AllBoundaryEdges.Add(Next->Value);
				RemoveCanonEdge(Walk, NextVal);
				Walk = NextVal;
			}
			int32 Count = AllBoundaryVerts.Num() - BdryStartIdx - 1;
			if (Count < 3) // failed to add a boundary, try again next loop
			{
				AllBoundaryVerts.SetNum(BdryStartIdx);
				AllBoundaryEdges.SetNum(BdryStartIdx);
				continue;
			}
			
			AllBoundaryVerts[BdryStartIdx] = Count; // record length of added chain
			AllBoundaryEdges[BdryStartIdx] = Count;
			NumHoles++;
		}
		if (NumHoles == 0)
		{
			return;
		}
		check(AllBoundaryVerts.Num() == AllBoundaryEdges.Num());

		/// Tag vertex connected components

		FDisjointSet VertComponents(Mesh.MaxVertexID());
		for (FIndex3i Tri : Mesh.TrianglesItr())
		{
			VertComponents.Union(Tri.A, Tri.B);
			VertComponents.Union(Tri.B, Tri.C);
		}

		FDynamicMeshMaterialAttribute* MaterialIDs = Mesh.Attributes()->GetMaterialID();

		/// 4. Compute stats on the holes: Their areas, and the neighboring component that best matches the hole normal

		TArray<int32> HoleComponents; // which connected component to connect the new hole geometry to
		HoleComponents.Init(-1, NumHoles);
		TArray<FVector3d> HoleNormals;
		HoleNormals.Init(FVector3d::ZeroVector, NumHoles);
		for (int32 BIdx = 0, CurBdry = 0; BIdx < AllBoundaryVerts.Num(); BIdx += AllBoundaryVerts[BIdx] + 1, CurBdry++)
		{
			int32 ChainLen = AllBoundaryVerts[BIdx];
			int32 EndIdx = BIdx + ChainLen + 1;
			check(ChainLen > 2 && EndIdx <= AllBoundaryVerts.Num());

			TMap<int32, FVector3d> ComponentToNormalMap;
			FVector3d HoleNormal(0, 0, 0);
			FVector3d POrigin = Mesh.GetVertex(AllBoundaryVerts[BIdx + 1]);
			
			for (int32 LastIdx = EndIdx - 1, Idx = BIdx + 1; Idx < EndIdx; LastIdx = Idx++)
			{
				if (LastIdx != BIdx + 1 && Idx != BIdx + 1)
				{
					int32 V0 = AllBoundaryVerts[LastIdx];
					int32 V1 = AllBoundaryVerts[Idx];
					FVector3d P0 = Mesh.GetVertex(V0);
					FVector3d P1 = Mesh.GetVertex(V1);
					FVector3d Edge = P1 - P0;
					FVector3d HoleTriNormal = Edge.Cross(P0 - POrigin);
					HoleNormal += HoleTriNormal;
				}

				int32 EID01 = AllBoundaryEdges[LastIdx];
				FDynamicMesh3::FEdge Edge01 = Mesh.GetEdge(EID01);
				int32 TID01 = Edge01.Tri.A;
				int32 MID01 = MaterialIDs->GetValue(TID01);
				if (MID01 >= 0 && (MID01 % 2) == 0)
				{
					continue; // skip "outside" materials, where UVs are less predictable
				}
				int32 Component = (int32)VertComponents.Find(Edge01.Vert.A);
				FVector3d Normal = Mesh.GetTriNormal(TID01);
				FVector3d* AccNormal = ComponentToNormalMap.Find(Component);
				if (AccNormal)
				{
					(*AccNormal) += Normal;
				}
				else
				{
					ComponentToNormalMap.Add(Component, Normal);
				}
			}
			int32 BestComponent = -1;
			double BestScore = -2; // Score is dot product of component avg triangle normal and hole normal
			bool bHasNormal = HoleNormal.Normalize(DBL_EPSILON);
			for (TPair<int32, FVector3d>& IDNormal : ComponentToNormalMap)
			{
				double Score;
				if (IDNormal.Value.Normalize(DBL_EPSILON))
				{
					Score = HoleNormal.Dot(IDNormal.Value);
				}
				else
				{
					Score = -1;
				}
				if (Score > BestScore)
				{
					BestScore = Score;
					BestComponent = IDNormal.Key;
				}
			}

			if (!bHasNormal && BestComponent > -1)
			{
				HoleNormal = ComponentToNormalMap[BestComponent];
			}
			// Note: BestComponent could be -1; currently we ignore the hole in this case.
			// This is the case where only the "outside" material is connected to the hole.
			// it tends to only happen for e.g. singular degenerate triangles, which are 
			// better to ignore.
			// If we'd instead like to fill these anyway, we could fall back to:
			//   BestComponent = VertComponents.Find(AllBoundaryVerts[BIdx + 1])
			// Or we could change the hole fill logic to create a completely
			// disconnected patch in this case.
			HoleComponents[CurBdry] = BestComponent;
			HoleNormals[CurBdry] = HoleNormal;
		}

		/// 5. Fill holes with new triangles

		int32 NumUVs = NumEnabledUVChannels(Mesh);

		TArray<int32> UseVIDs; // vertex IDs for the current hole
		for (int32 BIdx = 0, CurBdry = 0; BIdx < AllBoundaryVerts.Num(); BIdx += AllBoundaryVerts[BIdx] + 1, CurBdry++)
		{
			if (HoleComponents[CurBdry] == -1)
			{
				// hole was solely on the "outside" material; this can indicate an open boundary that isn't actually a hole
				// for now we skip these cases (See comment where HoleComponents are assigned, above)
				continue;
			}

			int32 ChainLen = AllBoundaryVerts[BIdx];
			int32 EndIdx = BIdx + ChainLen + 1;
			check(ChainLen > 2 && EndIdx <= AllBoundaryVerts.Num());

			TArray<FIndex3i> Triangles;
			TArray<FVector3d> VertexPositions;
			for (int Idx = BIdx + 1; Idx < EndIdx; Idx++)
			{
				VertexPositions.Add(Mesh.GetVertex(AllBoundaryVerts[Idx]));
			}
			PolygonTriangulation::TriangulateSimplePolygon(VertexPositions, Triangles, true);
			double HoleArea = 0;
			FVector3d LastNormal(0, 0, 0);
			const double FlippedThreshold = -.5; // Allow some angle deviation but reject hole fills that are too folded
			bool bHoleTriangulationIsFolded = false;
			for (FIndex3i Tri : Triangles)
			{
				double TriArea;
				FVector3d TriNormal = VectorUtil::NormalArea(VertexPositions[Tri.A], VertexPositions[Tri.B], VertexPositions[Tri.C], TriArea);
				HoleArea += TriArea;
				if (LastNormal.Dot(TriNormal) < FlippedThreshold)
				{
					bHoleTriangulationIsFolded = true;
					break;
				}
				if (TriArea != 0.0)
				{
					LastNormal = TriNormal;
				}
			}
			if (HoleArea < MinHoleArea || bHoleTriangulationIsFolded)
			{
				continue;
			}

			// find the hole material and edge matching the target vertex component
			int32 HoleMaterial = -1;
			FIndex2i HoleEdgeV; // reference edge for the hole
			for (int32 LastIdx = EndIdx - 1, Idx = BIdx + 1; Idx < EndIdx; LastIdx = Idx++)
			{
				int32 V0 = AllBoundaryVerts[LastIdx];
				int32 V1 = AllBoundaryVerts[Idx];
				int32 EID01 = AllBoundaryEdges[LastIdx];
				FDynamicMesh3::FEdge Edge = Mesh.GetEdge(EID01);
				int32 Component01 = (int32)VertComponents.Find((uint32)Edge.Vert.A);
				if (Edge.Tri.B == -1 && Component01 == HoleComponents[CurBdry]) // matching edge must still be a boundary edge
				{
					check(Edge.Vert.A != INDEX_NONE);
					HoleMaterial = MaterialIDs->GetValue(Edge.Tri.A);
					HoleEdgeV = Mesh.GetOrientedBoundaryEdgeV(EID01);
					break;
				}
			}

			// no boundary edge found with the target component ID
			if (HoleEdgeV.A == INDEX_NONE)
			{
				continue;
			}

			// find a basis for UV projection (per UV channel)
			FVector3d N = HoleNormals[CurBdry];
			FVector3d Origin = Mesh.GetVertex(HoleEdgeV.A);
			FVector3d T[8], B[8];
			{
				FVector3d Edge = Mesh.GetVertex(HoleEdgeV.B) - Origin;
				float EdgeLen = (float)Edge.Length();
				for (int32 UVIdx = 0; UVIdx < NumUVs; UVIdx++)
				{
					FVector2f UVA, UVB;
					GetUV(Mesh, HoleEdgeV.A, UVA, UVIdx);
					GetUV(Mesh, HoleEdgeV.B, UVB, UVIdx);
					FVector2f UVEdge = UVB - UVA;
					float UVEdgeLen = UVEdge.Length();
					float UVScale = UVEdgeLen / EdgeLen;
					double Angle = (double)FMath::Atan2(UVEdge.Y, UVEdge.X);
					UE::Geometry::FQuaterniond RotT(N, -Angle, false);
					UE::Geometry::FQuaterniond RotB(N, FMathd::HalfPi - Angle, false);
					T[UVIdx] = RotT * Edge * UVScale;
					B[UVIdx] = RotB * Edge * UVScale;

				}
			}

			// Indices of vertices to be used for the hole fill
			UseVIDs.Init(-1, ChainLen);

			// Find the vertices that match the target component -- we can re-use these directly
			for (int32 LastIdx = EndIdx - 1, Idx = BIdx + 1; Idx < EndIdx; LastIdx = Idx++)
			{
				int32 V0 = AllBoundaryVerts[LastIdx];
				int32 V1 = AllBoundaryVerts[Idx];

				int32 EID01 = AllBoundaryEdges[LastIdx];
				if (!Mesh.IsBoundaryEdge(EID01))
				{
					continue;
				}
				
				int32 C0 = LastIdx - BIdx - 1;
				int32 C1 = Idx - BIdx - 1;
				FIndex2i EdgeV = Mesh.GetOrientedBoundaryEdgeV(EID01);
				if (EdgeV.A >= 0 && HoleComponents[CurBdry] == (int32)VertComponents.Find(EdgeV.A))
				{
					UseVIDs[C0] = EdgeV.A;
					UseVIDs[C1] = EdgeV.B;
				}
			}

			// Copy color + tangents from the reference vertex (TODO: consider an average instead?)
			FVector4f HoleColor;
			FVector3f HoleTanU, HoleTanV;
			GetTangent(Mesh, HoleEdgeV.A, HoleTanU, HoleTanV);
			HoleColor = GetVertexColor(Mesh, HoleEdgeV.A);
			
			int32 PrevMaxVID = Mesh.MaxVertexID();
			// Create new vertices for all that weren't already assigned to an existing vertex
			for (int32 Idx = BIdx + 1; Idx < EndIdx; Idx++)
			{
				int32 C = Idx - BIdx - 1;
				if (UseVIDs[C] != -1)
				{
					continue;
				}

				int32 V = AllBoundaryVerts[Idx];
				FVector3d P = Mesh.GetVertex(V);
				UE::Geometry::FVertexInfo Info(P, (FVector3f)HoleNormals[CurBdry]);
				int32 NewV = Mesh.AppendVertex(Info);
				SetVertexColor(Mesh, NewV, HoleColor);
				SetTangent(Mesh, NewV, (FVector3f)HoleNormals[CurBdry], HoleTanU, HoleTanV);
				for (int32 UVIdx = 0; UVIdx < NumUVs; UVIdx++)
				{
					FVector3d Diff = P - Origin;
					FVector2f UV((float)Diff.Dot(T[UVIdx]), (float)Diff.Dot(B[UVIdx]));
					SetUV(Mesh, NewV, UV, UVIdx);
				}
				UseVIDs[C] = NewV;
			}

			for (FIndex3i Tri : Triangles)
			{
				int32 V0 = UseVIDs[Tri.A];
				int32 V1 = UseVIDs[Tri.B];
				int32 V2 = UseVIDs[Tri.C];
				if (V0 == V1 || V1 == V2 || V0 == V2)
				{
					continue;
				}
				FVector3d P0 = Mesh.GetVertex(V0);
				FVector3d P1 = Mesh.GetVertex(V1);
				FVector3d P2 = Mesh.GetVertex(V2);

				int32 TID = Mesh.AppendTriangle(V0, V1, V2);
				// append can fail from a non-manifold edge; in that case add new vertices and try again
				if (TID == FDynamicMesh3::NonManifoldID)
				{
					int32 SeparatedVIDs[]{ V0, V1, V2 };
					for (int32 SubIdx = 0; SubIdx < 3; SubIdx++)
					{
						if (SeparatedVIDs[SubIdx] < PrevMaxVID)
						{
							int32 OrigV = SeparatedVIDs[SubIdx];
							int32 NewV = Mesh.AppendVertex(Mesh, OrigV);
							SetTangent(Mesh, NewV, (FVector3f)HoleNormals[CurBdry], HoleTanU, HoleTanV);
							for (int32 UVIdx = 0; UVIdx < NumUVs; UVIdx++)
							{
								FVector2f UV;
								GetUV(Mesh, OrigV, UV, UVIdx);
								SetUV(Mesh, NewV, UV, UVIdx);
							}
							SeparatedVIDs[SubIdx] = NewV;
						}
					}
					TID = Mesh.AppendTriangle(SeparatedVIDs[0], SeparatedVIDs[1], SeparatedVIDs[2]);
				}
				if (TID > -1)
				{
					MaterialIDs->SetValue(TID, HoleMaterial);
					SetVisibility(Mesh, TID, true);
					SetInternal(Mesh, TID, true);
				}
			}
		}
	}

} // namespace AugmentedDynamicMesh


namespace
{
	/// Make a frame w/ Z aligned to the given normal
	/// and X aligned to the major axis most perpendicular to the normal
	FFrame3d AxisAlignedFrame(const FPlane& Plane)
	{
		FVector3d Origin = (FVector3d)Plane.GetOrigin();
		FVector3d Normal = (FVector3d)Plane.GetNormal();
		int32 MinIdx = UE::Geometry::MinAbsElementIndex(Normal);
		FFrame3d Frame(Origin, Normal);
		FVector3d Target(0, 0, 0);
		Target[MinIdx] = 1;
		Frame.ConstrainedAlignAxis(0, Target, Normal);
		return Frame;
	}


	// Replace any unset vertex colors with colors from coincident or neighboring vertices (if bPropagateFromNeighbors) or DefaultVertexColor
	void SetUnsetColors(FGeometryCollection* Collection, int32 FirstGeometryIndex, bool bPropagateFromNeighbors = true)
	{
		if (FirstGeometryIndex == INDEX_NONE)
		{
			return;
		}

		auto IsUnset = [](const FLinearColor& Color) -> bool
		{
			return Color.R < 0 || Color.G < 0 || Color.B < 0 || Color.A < 0;
		};
		constexpr float CopyColorDistance = 1e-03f;

		int32 NumGeo = Collection->NumElements(FGeometryCollection::GeometryGroup);
		
		if (bPropagateFromNeighbors)
		{
			// Get vertex range for the geometry in and after FirstGeometryIndex
			// Note: I think this will always just be the contiguous block of all vertices after the first geometry's start vertex,
			//       but explicitly find the range to be safe (and to make the code easier to adapt)
			int32 StartV = Collection->VertexStart[FirstGeometryIndex];
			int32 EndV = Collection->VertexCount[FirstGeometryIndex] + StartV;
			for (int32 GeoIdx = FirstGeometryIndex + 1; GeoIdx < NumGeo; ++GeoIdx)
			{
				int32 GeoStart = Collection->VertexStart[GeoIdx];
				if (GeoStart < StartV)
				{
					StartV = GeoStart;
				}
				else
				{
					int32 GeoEnd = GeoStart + Collection->VertexCount[GeoIdx];
					EndV = FMath::Max(GeoEnd, EndV);
				}
			}
			int32 NumV = EndV - StartV;

			// A generic representation of the full vertex graph
			// (To be split into connected components and solved as a linear system per component)
			struct FLink
			{
				int32 V[2]{ -1, -1 };
				float Wt = 1;

				FLink()
				{}

				FLink(int32 A, int32 B, float Wt) : V{ A, B }, Wt(Wt)
				{}

				static FLink Edge(int32 A, int32 B)
				{
					// Note: Currently just using constant weight (per half-edge)
					constexpr float EdgeWt = .5;
					return FLink(A, B, EdgeWt);
				}
			};
			TArray<FLink> Links; // connections between vertices -- note: always symmetric, duplicates are allowed
			Links.Reserve(NumV * 3);

			TArray<FVector4f> FixedColors;
			TArray<float> FixedColorWts;
			FixedColors.SetNumZeroed(NumV);
			FixedColorWts.SetNumZeroed(NumV);

			FVertexConnectedComponents Components(NumV); // Overall connected components (1 solve per component)
			FVertexConnectedComponents VtxComps(NumV); // Vertex components (1 group of overlapping vertices per component)

			for (int32 GeoIdx = FirstGeometryIndex; GeoIdx < NumGeo; ++GeoIdx)
			{
				int32 FStart = Collection->FaceStart[GeoIdx];
				int32 FEnd = FStart + Collection->FaceCount[GeoIdx];
				for (int32 FIdx = FStart; FIdx < FEnd; ++FIdx)
				{
					const FIntVector& Tri = Collection->Indices[FIdx];
					// Note: This assumes triangles always have either fully set or fully unset vertices, so we only need to check one vertex
					if (IsUnset(Collection->Color[Tri[0]]))
					{
						Components.ConnectVertices(Tri.X - StartV, Tri.Y - StartV);
						Components.ConnectVertices(Tri.Y - StartV, Tri.Z - StartV);
						Links.Add(FLink::Edge(Tri.X - StartV, Tri.Y - StartV));
						Links.Add(FLink::Edge(Tri.Y - StartV, Tri.Z - StartV));
						Links.Add(FLink::Edge(Tri.Z - StartV, Tri.X - StartV));
					}
				}
			}

			// put geometry in a shared coordinate space to spread color across geometry
			TArray<FTransform> GlobalTransformArray;
			GeometryCollectionAlgo::GlobalMatrices(Collection->Transform, Collection->Parent, GlobalTransformArray);
			TArray<FVector3f> GlobalVertices;
			GlobalVertices.SetNum(NumV);
			for (int32 Idx = StartV; Idx < EndV; Idx++)
			{
				GlobalVertices[Idx - StartV] = (FVector3f)GlobalTransformArray[Collection->BoneMap[Idx]].TransformPosition(FVector(Collection->Vertex[Idx]));
			}

			// Create a hash grid of vertices with unset colors, and connect components for overlapping unset vertices
			TPointHashGrid3f<int32> VertexHash(CopyColorDistance * 4.0f, INDEX_NONE);
			TArray<int32> Neighbors;

			for (int32 GeoIdx = FirstGeometryIndex; GeoIdx < NumGeo; ++GeoIdx)
			{
				int32 GeoStart = Collection->VertexStart[GeoIdx];
				int32 GeoEnd = GeoStart + Collection->VertexCount[GeoIdx];
				for (int32 Idx = GeoStart; Idx < GeoEnd; ++Idx)
				{
					FLinearColor Color = Collection->Color[Idx];
					if (IsUnset(Color))
					{
						if (Components.GetComponentSize(Idx - StartV) == 1) // isolated vertex (not in any triangle), just leave as default color
						{
							Collection->Color[Idx] = FLinearColor(AugmentedDynamicMesh::DefaultVertexColor);
							continue;
						}
						FVector3f Pt = GlobalVertices[Idx - StartV];

						int32 SetID = Components.GetComponent(Idx - StartV);

						Neighbors.Reset();
						VertexHash.FindPointsInBall(Pt, CopyColorDistance, [&GlobalVertices, &Pt, StartV](int32 OtherIdx)
							{
								return DistanceSquared(Pt, GlobalVertices[OtherIdx - StartV]);
							}, Neighbors);
						for (int32 NbrIdx : Neighbors)
						{
							VtxComps.ConnectVertices(Idx - StartV, NbrIdx - StartV);
							Components.ConnectVertices(SetID, NbrIdx - StartV);
						}
						VertexHash.InsertPointUnsafe(Idx, Pt);
					}
				}
			}

			TSet<int32> CanSolveComponents;

			// Fix colors on vertices that are coincident to set colors
			for (int32 GeoIdx = FirstGeometryIndex; GeoIdx < NumGeo; ++GeoIdx)
			{
				int32 GeoStart = Collection->VertexStart[GeoIdx];
				int32 GeoEnd = GeoStart + Collection->VertexCount[GeoIdx];
				for (int32 Idx = GeoStart; Idx < GeoEnd; ++Idx)
				{
					FLinearColor Color = Collection->Color[Idx];
					if (!IsUnset(Color))
					{
						FVector3f Pt = GlobalVertices[Idx - StartV];
						Neighbors.Reset();
						VertexHash.FindPointsInBall(Pt, CopyColorDistance, [&GlobalVertices, &Pt, StartV](int32 OtherIdx)
							{
								return DistanceSquared(Pt, GlobalVertices[OtherIdx - StartV]);
							}, Neighbors);
						for (int32 NbrIdx : Neighbors)
						{
							int32 Component = Components.GetComponent(NbrIdx - StartV);
							int32 ComponentSize = Components.GetComponentSize(NbrIdx - StartV);
							if (ComponentSize == 1)
							{
								Collection->Color[NbrIdx] = Color; // Isolated vertex, no solve needed
							}
							CanSolveComponents.Add(Component);
							int32 SharedVtx = VtxComps.GetComponent(NbrIdx - StartV);
							FixedColors[SharedVtx] += FVector4f(Color.R, Color.G, Color.B, Color.A);
							FixedColorWts[SharedVtx] += 1.0f;
						}
					}
				}
			}

			// Average color at any vertex w/ a fixed color
			for (int32 ColorIdx = 0; ColorIdx < FixedColors.Num(); ++ColorIdx)
			{
				float& Wt = FixedColorWts[ColorIdx];
				if (Wt > 0.0f)
				{
					FixedColors[ColorIdx] /= Wt;
					Wt = 1.0f;
				}
			}

			TArray<int32> ToComponentMap;
			ToComponentMap.Reserve(NumV);
			TArray<float> DiagonalWts;

			TArray<int32> ContigComponentVertices = Components.MakeContiguousComponentsArray(NumV);

			// Solve for vertex colors per component
			for (int32 ContigStart = 0, NextStart = -1; ContigStart < NumV; ContigStart = NextStart)
			{
				int32 ComponentID = Components.GetComponent(ContigComponentVertices[ContigStart]);
				int32 ComponentSize = Components.GetComponentSize(ComponentID);
				NextStart = ContigStart + ComponentSize;

				if (ComponentSize == 1)
				{
					continue;
				}
				if (!CanSolveComponents.Contains(ComponentID))
				{
					continue;
				}

				using FSparseMatf = Eigen::SparseMatrix<float, Eigen::ColMajor>;
				using FMatrixTripletf = Eigen::Triplet<float>;
				std::vector<FMatrixTripletf> EntryTriplets;

				// Make an index map for this component's vertices
				ToComponentMap.Init(-1, NumV);
				int32 NumToSolve = 0;
				for (int32 ContigIdx = ContigStart; ContigIdx < NextStart; ++ContigIdx)
				{
					int32 LocalIdx = ContigComponentVertices[ContigIdx];
					int32 GlobalIdx = LocalIdx + StartV;
					int32 SharedIdx = VtxComps.GetComponent(LocalIdx);
					if (FixedColorWts[SharedIdx] > 0)
					{
						FLinearColor Color(FixedColors[SharedIdx]);
						Collection->Color[GlobalIdx] = Color; // copy fixed color out
						continue; // has fixed color, do not include in solve
					}
					if (ToComponentMap[SharedIdx] != -1)
					{
						continue; // already mapped
					}

					ToComponentMap[SharedIdx] = NumToSolve;

					++NumToSolve;
				}

				if (NumToSolve == 0)
				{
					continue;
				}

				FSparseMatf SparseMatrix(NumToSolve, NumToSolve);
				Eigen::VectorXf X[4]{ Eigen::VectorXf(NumToSolve), Eigen::VectorXf(NumToSolve), Eigen::VectorXf(NumToSolve), Eigen::VectorXf(NumToSolve) };
				Eigen::VectorXf B[4]{ Eigen::VectorXf(NumToSolve), Eigen::VectorXf(NumToSolve), Eigen::VectorXf(NumToSolve), Eigen::VectorXf(NumToSolve) };
				for (int32 SubIdx = 0; SubIdx < 4; ++SubIdx)
				{
					B[SubIdx].setZero();
				}
				DiagonalWts.Reset(NumToSolve);
				DiagonalWts.SetNumZeroed(NumToSolve, EAllowShrinking::No);

				// Build the sparse matrix and rhs for the component
				for (const FLink& Link : Links)
				{
					int32 SharedA = VtxComps.GetComponent(Link.V[0]);
					int32 SharedB = VtxComps.GetComponent(Link.V[1]);
					int32 LocalA = ToComponentMap[SharedA];
					int32 LocalB = ToComponentMap[SharedB];
					if (LocalA == INDEX_NONE)
					{
						if (LocalB == INDEX_NONE)
						{
							continue;
						}
						Swap(SharedA, SharedB);
						Swap(LocalA, LocalB);
					}
					if (LocalB == INDEX_NONE)
					{
						if (FixedColorWts[SharedB] > 0)
						{
							FVector4f BVal = FixedColors[SharedB] * Link.Wt;
							for (int32 SubIdx = 0; SubIdx < 4; ++SubIdx)
							{
								B[SubIdx][LocalA] += BVal[SubIdx];
							}
							DiagonalWts[LocalA] += Link.Wt;
						}
					}
					else
					{
						EntryTriplets.push_back(FMatrixTripletf(LocalA, LocalB, -Link.Wt));
						EntryTriplets.push_back(FMatrixTripletf(LocalB, LocalA, -Link.Wt));
						DiagonalWts[LocalA] += Link.Wt;
						DiagonalWts[LocalB] += Link.Wt;
					}
				}
				for (int32 Idx = 0; Idx < NumToSolve; ++Idx)
				{
					EntryTriplets.push_back(FMatrixTripletf(Idx, Idx, DiagonalWts[Idx]));
				}

				// Solve linear system for internal colors
				SparseMatrix.setFromTriplets(EntryTriplets.begin(), EntryTriplets.end());
				SparseMatrix.makeCompressed();
				
				Eigen::SparseLU<FSparseMatf, Eigen::COLAMDOrdering<int>> MatrixSolver;
				MatrixSolver.isSymmetric(true);
				MatrixSolver.analyzePattern(SparseMatrix);
				MatrixSolver.factorize(SparseMatrix);

				ParallelFor(4, [&](int32 Idx)
					{
						X[Idx] = MatrixSolver.solve(B[Idx]);
					});

				// Copy solved colors out
				for (int32 ContigIdx = ContigStart; ContigIdx < NextStart; ++ContigIdx)
				{
					int32 LocalIdx = ContigComponentVertices[ContigIdx];
					int32 GlobalIdx = LocalIdx + StartV;
					int32 SharedIdx = VtxComps.GetComponent(LocalIdx);
					int32 CompIdx = ToComponentMap[SharedIdx];
					if (CompIdx != INDEX_NONE)
					{
						FVector4f SolvedColor;
						for (int32 SubIdx = 0; SubIdx < 4; ++SubIdx)
						{
							// Note: make sure the solved color is non-negative, so solver error cannot make the color 'unset'
							SolvedColor[SubIdx] = FMath::Max(X[SubIdx][CompIdx], 0.f);
						}

						Collection->Color[GlobalIdx] = FLinearColor(SolvedColor);
					}
				}
			}
		}

		// Replace unset color with default color
		for (int32 GeoIdx = FirstGeometryIndex; GeoIdx < NumGeo; ++GeoIdx)
		{
			int32 VStart = Collection->VertexStart[GeoIdx];
			int32 VEnd = VStart + Collection->VertexCount[GeoIdx];
			for (int32 Idx = VStart; Idx < VEnd; ++Idx)
			{
				if (IsUnset(Collection->Color[Idx]))
				{
					Collection->Color[Idx] = FLinearColor(AugmentedDynamicMesh::DefaultVertexColor);
				}
			}
		}
	}
}


void SetGeometryCollectionAttributes(FDynamicMesh3& Mesh, int32 NumUVLayers)
{
	AugmentedDynamicMesh::Augment(Mesh, NumUVLayers);
}

void ClearCustomGeometryCollectionAttributes(UE::Geometry::FDynamicMesh3& Mesh)
{
	Mesh.DiscardVertexNormals();
	
	Mesh.Attributes()->RemoveAttribute(AugmentedDynamicMesh::ColorAttribName);
	Mesh.Attributes()->RemoveAttribute(AugmentedDynamicMesh::TangentUAttribName);
	Mesh.Attributes()->RemoveAttribute(AugmentedDynamicMesh::TangentVAttribName);
	Mesh.Attributes()->RemoveAttribute(AugmentedDynamicMesh::VisibleAttribName);
	Mesh.Attributes()->RemoveAttribute(AugmentedDynamicMesh::InternalAttribName);

	AugmentedDynamicMesh::EnableUVChannels(Mesh, 0, false, true); // set 0 UV channels to remove UV attributes
}


FCellMeshes::FCellMeshes(int32 NumUVLayersIn, FRandomStream& RandomStream, const FPlanarCells& Cells, FAxisAlignedBox3d DomainBounds, double Grout, double ExtendDomain, bool bIncludeOutsideCell)
{
	Init(NumUVLayersIn, RandomStream, Cells, DomainBounds, Grout, ExtendDomain, bIncludeOutsideCell);
}


FCellMeshes::FCellMeshes(int32 NumUVLayersIn, const FDynamicMesh3& SingleCutter, const FInternalSurfaceMaterials& Materials, TOptional<FTransform> Transform)
{
	NumUVLayers = NumUVLayersIn;
	SetNumCells(2);

	CellMeshes[0].AugMesh = SingleCutter;

	if (Transform.IsSet())
	{
		MeshTransforms::ApplyTransform(CellMeshes[0].AugMesh, FTransformSRT3d(Transform.GetValue()), true);
	}

	// Augment mesh if needed
	if (!AugmentedDynamicMesh::IsAugmented(CellMeshes[0].AugMesh))
	{
		AugmentedDynamicMesh::Augment(CellMeshes[0].AugMesh, NumUVLayers);
		// transfer overlay attributes to 'augmented attribute' format
		AugmentedDynamicMesh::SplitOverlayAttributesToPerVertex(CellMeshes[0].AugMesh);
	}

	// Make sure color is unset
	for (int VID : CellMeshes[0].AugMesh.VertexIndicesItr())
	{
		AugmentedDynamicMesh::SetVertexColor(CellMeshes[0].AugMesh, VID, AugmentedDynamicMesh::UnsetVertexColor);
	}

	// first mesh is the same as the second mesh, but will be subtracted b/c it's the "outside cell"
	// TODO: special case this logic so we don't have to hold two copies of the exact same mesh!
	CellMeshes[1].AugMesh = CellMeshes[0].AugMesh;
	OutsideCellIndex = 1;

}


// Special function to just make the "grout" part of the planar mesh cells
// Used to make the multi-plane cuts with grout easier to implement
void FCellMeshes::MakeOnlyPlanarGroutCell(const FPlanarCells& Cells, FAxisAlignedBox3d DomainBounds, double Grout)
{
	CellMeshes.Reset();

	if (!ensure(Grout > 0) || !ensure(Cells.IsInfinitePlane()))
	{
		return;
	}

	float GlobalUVScale = Cells.InternalSurfaceMaterials.GlobalUVScale;
	if (!ensure(GlobalUVScale > 0))
	{
		GlobalUVScale = 1;
	}

	SetNumCells(1);

	bool bNoise = Cells.InternalSurfaceMaterials.NoiseSettings.IsSet();

	double ExtendDomain = bNoise ? Cells.InternalSurfaceMaterials.NoiseSettings->Amplitude : 0;
	DomainBounds.Expand(ExtendDomain);

	CreateMeshesForSinglePlane(Cells, DomainBounds, bNoise, GlobalUVScale, Grout, true);

	for (FCellInfo& CellInfo : CellMeshes)
	{
		AugmentedDynamicMesh::SetDefaultAttributes(CellInfo.AugMesh, Cells.InternalSurfaceMaterials.bGlobalVisibility);
	}
}


void FCellMeshes::RemeshForNoise(FDynamicMesh3& Mesh, EEdgeRefineFlags EdgeFlags, double TargetEdgeLen)
{
	FQueueRemesher Remesh(&Mesh);
	Remesh.bPreventNormalFlips = true;
	FMeshConstraints Constraints;

	FMeshBoundaryLoops Boundary(&Mesh);
	int LoopCount = Boundary.GetLoopCount();
	if (!ensureMsgf(LoopCount == 1, TEXT("Expected to remesh a patch with a single boundary but found %d boundary loops"), LoopCount))
	{
		if (LoopCount == 0)
		{
			return;
		}
	}

	for (int VID : Mesh.VertexIndicesItr())
	{
		FVertexConstraint FullyConstrain(true, false, VID);
		Constraints.SetOrUpdateVertexConstraint(VID, FullyConstrain);
	}


	FEdgeConstraint EdgeConstraint(EdgeFlags);
	for (int EID : Boundary[0].Edges)
	{
		Constraints.SetOrUpdateEdgeConstraint(EID, EdgeConstraint);
	}
	Remesh.SetExternalConstraints(Constraints);
	Remesh.SetTargetEdgeLength(TargetEdgeLen);
	Remesh.Precompute();
	Remesh.FastestRemesh();
}


FVector3d FCellMeshes::NoiseDisplacement(const FVector3d& Pos, const FNoiseSettings& Settings)
{
	return FVector3d(Settings.NoiseVector(FVector(Pos), NoiseOffsets));
}


void FCellMeshes::ApplyNoise(FDynamicMesh3& Mesh, FVector3d Normal, const FNoiseSettings& Settings, bool bProjectBoundariesToNormal)
{
	double Amplitude = (double)Settings.Amplitude;
	double Frequency = (double)Settings.Frequency;
	int32 Octaves = Settings.Octaves;
	FVector3d Z = Normal * Amplitude;

	for (int VID : Mesh.VertexIndicesItr())
	{
		FVector3d Pos = Mesh.GetVertex(VID);
		FVector3d Displacement = NoiseDisplacement(Pos, Settings);
		if (bProjectBoundariesToNormal || !Mesh.IsBoundaryVertex(VID))
		{
			// project displacement onto the normal direction
			Displacement = Normal * Displacement.Dot(Normal);
		}


		Mesh.SetVertex(VID, Pos + Displacement);
	}
}


void FCellMeshes::Init(int32 NumUVLayersIn, FRandomStream& RandomStream, const FPlanarCells& Cells, FAxisAlignedBox3d DomainBounds, double Grout, double ExtendDomain, bool bIncludeOutsideCell)
{
	NumUVLayers = NumUVLayersIn;
	InitEmpty(RandomStream);

	float GlobalUVScale = Cells.InternalSurfaceMaterials.GlobalUVScale;
	if (!ensure(GlobalUVScale > 0))
	{
		GlobalUVScale = 1;
	}

	int NumCells = Cells.NumCells;
	bool bHasGroutCell = Grout > 0;
	if (bIncludeOutsideCell && !Cells.IsInfinitePlane())
	{
		OutsideCellIndex = NumCells;
		NumCells++;
	}

	SetNumCells(NumCells);

	bool bNoise = Cells.InternalSurfaceMaterials.NoiseSettings.IsSet();
	if (bNoise)
	{
		ExtendDomain += Cells.InternalSurfaceMaterials.NoiseSettings->Amplitude;
	}
	DomainBounds.Expand(ExtendDomain);

	// special handling for the infinite plane case; we need to adapt this to be a closed volume
	if (Cells.IsInfinitePlane())
	{
		CreateMeshesForSinglePlane(Cells, DomainBounds, bNoise, GlobalUVScale, Grout, false);
	}
	else
	{
		if (!bNoise) // bounded cells w/ no noise
		{
			CreateMeshesForBoundedPlanesWithoutNoise(NumCells, Cells, DomainBounds, bNoise, GlobalUVScale);
		}
		else // bounded cells with noise -- make each boundary plane separately so we can remesh them w/ noise vertices
		{
			CreateMeshesForBoundedPlanesWithNoise(NumCells, Cells, DomainBounds, bNoise, GlobalUVScale);
		}
		ApplyGeneralGrout(Grout);
	}

	// TODO: self-union on cells when it makes sense to do so (for non-single-plane inputs w/ high noise or possible untracked adjacencies)
	/*for (FCellInfo& CellInfo : CellMeshes)
	{
		FMeshSelfUnion SelfUnion(&CellInfo.AugMesh);
		// TODO: need to have an option in SelfUnion to not weld edges
		SelfUnion.Compute();
	}*/

	for (FCellInfo& CellInfo : CellMeshes)
	{
		AugmentedDynamicMesh::SetDefaultAttributes(CellInfo.AugMesh, Cells.InternalSurfaceMaterials.bGlobalVisibility);
	}
}


void FCellMeshes::ApplyGeneralGrout(double Grout)
{
	if (Grout <= 0)
	{
		return;
	}

	// apply grout to all cells
	for (int MeshIdx = 0; MeshIdx < CellMeshes.Num(); MeshIdx++)
	{
		if (MeshIdx == OutsideCellIndex)
		{
			continue;
		}

		FDynamicMesh3& Mesh = CellMeshes[MeshIdx].AugMesh;
		// TODO: scale from mesh center of mass instead of the vertex centroid?
		FVector3d VertexCentroid(0, 0, 0);
		for (FVector3d V : Mesh.VerticesItr())
		{
			VertexCentroid += V;
		}
		VertexCentroid /= (double)Mesh.VertexCount();
		FAxisAlignedBox3d Bounds = Mesh.GetBounds(true);
		double BoundsSize = Bounds.MaxDim();
		// currently just scale the meshes down so they leave half-a-grout worth of space on their longest axis
		// or delete the mesh if it's so small that that would require a negative scale
		// TODO: consider instead computing a true offset mesh
		//  (note that we don't currently have a good UV-preserving+sharp-edge-preserving way to do that)
		double ScaleFactor = (BoundsSize - Grout * .5) / BoundsSize;
		if (ScaleFactor < FMathd::ZeroTolerance * 1000)
		{
			// if the grout scale factor would be ~zero or negative, just clear the mesh instead
			Mesh.Clear();
			AugmentedDynamicMesh::Augment(Mesh, NumUVLayers);
		}
		else
		{
			MeshTransforms::Scale(Mesh, FVector3d::One() * ScaleFactor, VertexCentroid);
		}
	}

	// create outside cell (if there is room for it) by appending all the other meshes
	if (OutsideCellIndex != -1)
	{
		FDynamicMesh3& OutsideMesh = CellMeshes[OutsideCellIndex].AugMesh;
		OutsideMesh.Clear();
		AugmentedDynamicMesh::Augment(OutsideMesh, NumUVLayers);
		FDynamicMeshEditor OutsideMeshEditor(&OutsideMesh);
		for (int MeshIdx = 0; MeshIdx < CellMeshes.Num(); MeshIdx++)
		{
			if (MeshIdx == OutsideCellIndex)
			{
				continue;
			}
			FMeshIndexMappings IndexMaps;
			OutsideMeshEditor.AppendMesh(&CellMeshes[MeshIdx].AugMesh, IndexMaps);
		}
	}
}


void FCellMeshes::AppendMesh(FDynamicMesh3& Base, FDynamicMesh3& ToAppend, bool bFlipped)
{
	FDynamicMeshEditor Editor(&Base);
	FMeshIndexMappings Mapping;
	Editor.AppendMesh(&ToAppend, Mapping);
	if (bFlipped)
	{
		for (int TID : ToAppend.TriangleIndicesItr())
		{
			Base.ReverseTriOrientation(Mapping.GetNewTriangle(TID));
		}
		for (int VID : ToAppend.VertexIndicesItr())
		{
			int BaseVID = Mapping.GetNewVertex(VID);
			Base.SetVertexNormal(BaseVID, -Base.GetVertexNormal(BaseVID));
		}
	}
}




void FCellMeshes::CreateMeshesForBoundedPlanesWithoutNoise(int NumCells, const FPlanarCells& Cells, const FAxisAlignedBox3d& DomainBounds, bool bNoise, double GlobalUVScale)
{
	for (int32 PlaneIdx = 0; PlaneIdx < Cells.PlaneCells.Num(); PlaneIdx++)
	{
		const TPair<int32, int32>& CellPair = Cells.PlaneCells[PlaneIdx];
		FDynamicMesh3* Meshes[2]{ &CellMeshes[CellPair.Key].AugMesh, nullptr };
		int32 OtherCell = CellPair.Value < 0 ? OutsideCellIndex : CellPair.Value;
		int NumMeshes = OtherCell < 0 ? 1 : 2;
		if (NumMeshes == 2)
		{
			Meshes[1] = &CellMeshes[OtherCell].AugMesh;
		}

		const TArray<int>& PlaneBoundary = Cells.PlaneBoundaries[PlaneIdx];
		FVector3f Normal(Cells.Planes[PlaneIdx].GetNormal());
		FFrame3d PlaneFrame = AxisAlignedFrame(Cells.Planes[PlaneIdx]);
		FVertexInfo PlaneVertInfo;
		PlaneVertInfo.bHaveC = true;
		PlaneVertInfo.bHaveUV = false;
		PlaneVertInfo.bHaveN = true;
		int VertStart[2]{ -1, -1 };
		for (int MeshIdx = 0; MeshIdx < NumMeshes; MeshIdx++)
		{
			PlaneVertInfo.Normal = Normal;
			if (MeshIdx == 1 && OtherCell != OutsideCellIndex)
			{
				PlaneVertInfo.Normal *= -1.0f;
			}
			VertStart[MeshIdx] = Meshes[MeshIdx]->MaxVertexID();
			FVector2f MinUV(FMathf::MaxReal, FMathf::MaxReal);
			for (int BoundaryVertex : PlaneBoundary)
			{
				FVector3d Position = FVector3d(Cells.PlaneBoundaryVertices[BoundaryVertex]);
				FVector2f UV = FVector2f(PlaneFrame.ToPlaneUV(Position));
				MinUV.X = FMathf::Min(UV.X, MinUV.X);
				MinUV.Y = FMathf::Min(UV.Y, MinUV.Y);
			}
			for (int BoundaryVertex : PlaneBoundary)
			{
				PlaneVertInfo.Position = FVector3d(Cells.PlaneBoundaryVertices[BoundaryVertex]);
				FVector2f UV = (FVector2f(PlaneFrame.ToPlaneUV(PlaneVertInfo.Position)) - MinUV) * static_cast<float>(GlobalUVScale);
				int VID = Meshes[MeshIdx]->AppendVertex(PlaneVertInfo);
				AugmentedDynamicMesh::SetVertexColor(*Meshes[MeshIdx], VID, AugmentedDynamicMesh::UnsetVertexColor);
				AugmentedDynamicMesh::SetAllUV(*Meshes[MeshIdx], VID, UV, NumUVLayers);
			}
		}

		int MID = PlaneToMaterial(PlaneIdx);
		if (Cells.AssumeConvexCells)
		{
			// put a fan
			for (int V0 = 0, V1 = 1, V2 = 2; V2 < PlaneBoundary.Num(); V1 = V2++)
			{
				for (int MeshIdx = 0; MeshIdx < NumMeshes; MeshIdx++)
				{
					int Offset = VertStart[MeshIdx];
					FIndex3i Tri(V0 + Offset, V1 + Offset, V2 + Offset);
					if (MeshIdx == 1 && OtherCell != OutsideCellIndex)
					{
						Swap(Tri.B, Tri.C);
					}
					int TID = Meshes[MeshIdx]->AppendTriangle(Tri);
					if (ensure(TID > -1))
					{
						Meshes[MeshIdx]->Attributes()->GetMaterialID()->SetNewValue(TID, MID);
					}
				}
			}
		}
		else // cells may not be convex; cannot triangulate w/ fan
		{
			// Delaunay triangulate
			FPolygon2f Polygon;
			for (int V = 0; V < PlaneBoundary.Num(); V++)
			{
				FVector2f UV;
				AugmentedDynamicMesh::GetUV(*Meshes[0], VertStart[0] + V, UV, 0);
				Polygon.AppendVertex(UV);
			}

			FGeneralPolygon2f GeneralPolygon(Polygon);
			FConstrainedDelaunay2f Triangulation;
			Triangulation.FillRule = FConstrainedDelaunay2f::EFillRule::NonZero;
			Triangulation.Add(GeneralPolygon);
			Triangulation.Triangulate();

			for (int MeshIdx = 0; MeshIdx < NumMeshes; MeshIdx++)
			{
				int Offset = VertStart[MeshIdx];
				for (FIndex3i Triangle : Triangulation.Triangles)
				{
					Triangle.A += Offset;
					Triangle.B += Offset;
					Triangle.C += Offset;
					if (MeshIdx == 1 && OtherCell != OutsideCellIndex)
					{
						Swap(Triangle.B, Triangle.C);
					}
					int TID = Meshes[MeshIdx]->AppendTriangle(Triangle);
					if (ensure(TID > -1))
					{
						Meshes[MeshIdx]->Attributes()->GetMaterialID()->SetNewValue(TID, MID);
					}
				}
			}
		}
	}
}


double FCellMeshes::GetSafeNoiseSpacing(float SurfaceArea, float TargetSpacing)
{
	double MaxVerts = 1000000; // try to avoid creating more than ~ a million new vertices
	double MinEdgeLen = FMathd::Sqrt((double)SurfaceArea / MaxVerts);
	double Spacing = FMath::Max3(.001, MinEdgeLen, (double)TargetSpacing);
	if (Spacing > TargetSpacing)
	{
		UE_LOGF(LogPlanarCut, Warning,
			"Requested spacing of noise points (surface resolution) of %f would require too many added vertices; Using %f instead.",
			TargetSpacing, Spacing);
	}
	return Spacing;
}

void FCellMeshes::CreateMeshesForBoundedPlanesWithNoise(int NumCells, const FPlanarCells& Cells, const FAxisAlignedBox3d& DomainBounds, bool bNoise, double GlobalUVScale)
{
	TArray<FDynamicMesh3> PlaneMeshes;
	PlaneMeshes.SetNum(Cells.Planes.Num());
	FName OriginalPositionAttribute = "OriginalPosition";
	for (FDynamicMesh3& PlaneMesh : PlaneMeshes)
	{
		AugmentedDynamicMesh::EnableUVChannels(PlaneMesh, NumUVLayers);
		PlaneMesh.EnableVertexNormals(FVector3f::UnitZ());
		PlaneMesh.EnableAttributes();
		PlaneMesh.Attributes()->EnableMaterialID();
		PlaneMesh.Attributes()->AttachAttribute(OriginalPositionAttribute, new TDynamicMeshVertexAttribute<double, 3>(&PlaneMesh));
	}

	struct FPlaneIdxAndFlip
	{
		int32 PlaneIdx;
		bool bIsFlipped;
	};
	TArray<TArray<FPlaneIdxAndFlip>> CellPlanes; // per cell, the planes that border that cell
	CellPlanes.SetNum(NumCells);

	for (int32 PlaneIdx = 0; PlaneIdx < Cells.PlaneCells.Num(); PlaneIdx++)
	{
		const TPair<int32, int32>& CellPair = Cells.PlaneCells[PlaneIdx];
		int32 OtherCell = CellPair.Value < 0 ? OutsideCellIndex : CellPair.Value;
		if (ensure(CellPlanes.IsValidIndex(CellPair.Key)))
		{
			CellPlanes[CellPair.Key].Add({ PlaneIdx, false });
		}
		if (CellPlanes.IsValidIndex(OtherCell))
		{
			CellPlanes[OtherCell].Add({ PlaneIdx, true });
		}
	}

	// heuristic to protect against creating too many vertices on remeshing
	float TotalArea = 0;
	for (int32 PlaneIdx = 0; PlaneIdx < Cells.Planes.Num(); PlaneIdx++)
	{
		const TArray<int>& PlaneBoundary = Cells.PlaneBoundaries[PlaneIdx];
		const FVector& V0 = Cells.PlaneBoundaryVertices[PlaneBoundary[0]];
		FVector AreaVec = FVector::ZeroVector;
		for (int32 V1Idx = 1, V2Idx = 2; V2Idx < PlaneBoundary.Num(); V1Idx = V2Idx++)
		{
			const FVector& V1 = Cells.PlaneBoundaryVertices[PlaneBoundary[V1Idx]];
			const FVector& V2 = Cells.PlaneBoundaryVertices[PlaneBoundary[V2Idx]];
			AreaVec += (V1 - V0) ^ (V2 - V1);
		}
		TotalArea += static_cast<float>(AreaVec.Size());
	}
	double Spacing = GetSafeNoiseSpacing(TotalArea, Cells.InternalSurfaceMaterials.NoiseSettings->PointSpacing);

	ParallelFor(Cells.Planes.Num(), [this, OriginalPositionAttribute, &PlaneMeshes, &Cells, GlobalUVScale, Spacing](int32 PlaneIdx)
		{
			FDynamicMesh3& Mesh = PlaneMeshes[PlaneIdx];
			const TArray<int>& PlaneBoundary = Cells.PlaneBoundaries[PlaneIdx];
			FVector3f Normal(Cells.Planes[PlaneIdx].GetNormal());
			FFrame3d PlaneFrame(Cells.Planes[PlaneIdx]);
			FVertexInfo PlaneVertInfo;
			PlaneVertInfo.bHaveC = true;
			PlaneVertInfo.bHaveUV = false;
			PlaneVertInfo.bHaveN = true;
			PlaneVertInfo.Normal = Normal;
			// UVs will be set below, after noise is added
			// UnsetVertexColor will be set below

			FPolygon2f Polygon;
			for (int BoundaryVertex : PlaneBoundary)
			{
				PlaneVertInfo.Position = FVector3d(Cells.PlaneBoundaryVertices[BoundaryVertex]);
				Polygon.AppendVertex((FVector2f)PlaneFrame.ToPlaneUV(PlaneVertInfo.Position));
				Mesh.AppendVertex(PlaneVertInfo);
			}

			// we do a CDT here to give a slightly better start to remeshing; we could try simple ear clipping instead
			FGeneralPolygon2f GeneralPolygon(Polygon);
			FConstrainedDelaunay2f Triangulation;
			Triangulation.FillRule = FConstrainedDelaunay2f::EFillRule::NonZero;
			Triangulation.Add(GeneralPolygon);
			Triangulation.Triangulate();
			if (Triangulation.Triangles.Num() == 0) // fall back to ear clipping if the triangulation came back empty
			{
				PolygonTriangulation::TriangulateSimplePolygon(Polygon.GetVertices(), Triangulation.Triangles, false);
			}
			if (ensure(Triangulation.Triangles.Num() > 0))
			{
				int MID = PlaneToMaterial(PlaneIdx);
				for (FIndex3i Triangle : Triangulation.Triangles)
				{
					int TID = Mesh.AppendTriangle(Triangle);
					if (ensure(TID > -1))
					{
						Mesh.Attributes()->GetMaterialID()->SetNewValue(TID, MID);
					}
				}

				RemeshForNoise(Mesh, EEdgeRefineFlags::SplitsOnly, Spacing);
				TDynamicMeshVertexAttribute<double, 3>* OriginalPosns =
					static_cast<TDynamicMeshVertexAttribute<double, 3>*>(Mesh.Attributes()->GetAttachedAttribute(OriginalPositionAttribute));
				for (int VID : Mesh.VertexIndicesItr())
				{
					OriginalPosns->SetValue(VID, Mesh.GetVertex(VID));
				}
				ApplyNoise(Mesh, FVector3d(Normal), Cells.InternalSurfaceMaterials.NoiseSettings.GetValue());

				FMeshNormals::QuickComputeVertexNormals(Mesh);
			}
		}, EParallelForFlags::None);

	for (int CellIdx = 0; CellIdx < NumCells; CellIdx++)
	{
		FCellInfo& CellInfo = CellMeshes[CellIdx];
		FDynamicMesh3& Mesh = CellInfo.AugMesh;
		Mesh.Attributes()->AttachAttribute(OriginalPositionAttribute, new TDynamicMeshVertexAttribute<double, 3>(&Mesh));
		bool bFlipForOutsideCell = CellIdx == OutsideCellIndex; // outside cell will be subtracted, and needs all planes flipped vs normal
		for (FPlaneIdxAndFlip PlaneInfo : CellPlanes[CellIdx])
		{
			AppendMesh(Mesh, PlaneMeshes[PlaneInfo.PlaneIdx], PlaneInfo.bIsFlipped ^ bFlipForOutsideCell);
		}
	}

	// resolve self-intersections

	// build hash grid of mesh vertices so we correspond all same-pos vertices across touching meshes
	TPointHashGrid3d<FIndex2i> MeshesVertices(FMathd::ZeroTolerance * 1000, FIndex2i::Invalid());
	for (int CellIdx = 0; CellIdx < NumCells; CellIdx++)
	{
		FCellInfo& CellInfo = CellMeshes[CellIdx];
		FDynamicMesh3& Mesh = CellInfo.AugMesh;
		for (int VID : Mesh.VertexIndicesItr())
		{
			MeshesVertices.InsertPointUnsafe(FIndex2i(CellIdx, VID), Mesh.GetVertex(VID));
		}
	}

	// repeatedly detect and resolve collisions until there are no more (or give up after too many iterations)
	TArray<bool> CellUnmoved; CellUnmoved.Init(false, NumCells);
	const int MaxIters = 10;
	for (int Iters = 0; Iters < MaxIters; Iters++)
	{
		struct FUpdate
		{
			FIndex2i Tris;
			TArray<FIndex2i> IDs;
			FUpdate(int TriA = -1, int TriB = -1) : Tris(TriA, TriB)
			{}
		};

		// todo: can parallelize?
		TArray<TArray<FUpdate>> Updates; Updates.SetNum(NumCells);
		bool bAnyUpdatesNeeded = false;
		for (int CellIdx = 0; CellIdx < NumCells; CellIdx++)
		{
			if (CellUnmoved[CellIdx])
			{
				// if nothing moved since last time we resolved self intersections on this cell, don't need to process again
				continue;
			}
			FDynamicMesh3& Mesh = CellMeshes[CellIdx].AugMesh;
			FDynamicMeshAABBTree3 CellTree(&Mesh, true);
			MeshIntersection::FIntersectionsQueryResult Intersections = CellTree.FindAllSelfIntersections(true);
			for (MeshIntersection::FSegmentIntersection& Seg : Intersections.Segments)
			{
				// manually check for shared edges by vertex position because they might not be topologically connected
				FIndex3i Tri[2]{ Mesh.GetTriangle(Seg.TriangleID[0]), Mesh.GetTriangle(Seg.TriangleID[1]) };
				int MatchedVertices = 0;
				for (int T0SubIdx = 0; T0SubIdx < 3; T0SubIdx++)
				{
					FVector3d V0 = Mesh.GetVertex(Tri[0][T0SubIdx]);
					for (int T1SubIdx = 0; T1SubIdx < 3; T1SubIdx++)
					{
						FVector3d V1 = Mesh.GetVertex(Tri[1][T1SubIdx]);
						if (DistanceSquared(V0, V1) < FMathd::ZeroTolerance)
						{
							MatchedVertices++;
							break;
						}
					}
				}
				// no shared vertices: treat as a real collision
				// (TODO: only skip shared edges? will need to do something to avoid shared vertices becoming collisions)
				if (MatchedVertices < 1)
				{
					bAnyUpdatesNeeded = true;
					FUpdate& Update = Updates[CellIdx].Emplace_GetRef(Seg.TriangleID[0], Seg.TriangleID[1]);
					for (int TriIdx = 0; TriIdx < 2; TriIdx++)
					{
						for (int VSubIdx = 0; VSubIdx < 3; VSubIdx++)
						{
							int VIdx = Tri[TriIdx][VSubIdx];
							FVector3d P = Mesh.GetVertex(VIdx);
							FIndex2i IDs(CellIdx, VIdx);
							MeshesVertices.FindPointsInBall(P, FMathd::ZeroTolerance, [this, P](FIndex2i IDs)
								{
									FVector3d Pos = CellMeshes[IDs.A].AugMesh.GetVertex(IDs.B);
									return DistanceSquared(P, Pos);
								}, Update.IDs);
						}
					}
				}
			}
		}
		if (!bAnyUpdatesNeeded)
		{
			break;
		}
		for (int CellIdx = 0; CellIdx < NumCells; CellIdx++)
		{
			CellUnmoved[CellIdx] = true;
		}

		// todo: maybe can parallelize if movements are not applied until after?
		for (int CellIdx = 0; CellIdx < NumCells; CellIdx++)
		{
			FDynamicMesh3& Mesh = CellMeshes[CellIdx].AugMesh;
			TDynamicMeshVertexAttribute<double, 3>* OriginalPosns =
				static_cast<TDynamicMeshVertexAttribute<double, 3>*>(Mesh.Attributes()->GetAttachedAttribute(OriginalPositionAttribute));
			auto InterpVert = [&Mesh, &OriginalPosns](int VID, double t)
			{
				FVector3d OrigPos, NoisePos;
				OriginalPosns->GetValue(VID, OrigPos);
				NoisePos = Mesh.GetVertex(VID);
				return UE::Geometry::Lerp(OrigPos, NoisePos, t);
			};
			auto InterpTri = [&Mesh, &InterpVert](int TID, double t)
			{
				FIndex3i TriVIDs = Mesh.GetTriangle(TID);
				FTriangle3d Tri;
				for (int i = 0; i < 3; i++)
				{
					Tri.V[i] = InterpVert(TriVIDs[i], t);
				}
				return Tri;
			};
			auto TestIntersection = [&InterpTri](int TIDA, int TIDB, double t)
			{
				FIntrTriangle3Triangle3d TriTri(InterpTri(TIDA, t), InterpTri(TIDB, t));
				return TriTri.Find();
			};
			// resolve tri-tri intersections on this cell's mesh (moving associated verts on other meshes as needed also)
			for (FUpdate& Update : Updates[CellIdx])
			{
				double tsafe = 0;
				double tbad = 1;
				if (!TestIntersection(Update.Tris.A, Update.Tris.B, tbad))
				{
					continue;
				}
				for (int SearchSteps = 0; SearchSteps < 4; SearchSteps++)
				{
					double tmid = (tsafe + tbad) * .5;
					if (TestIntersection(Update.Tris.A, Update.Tris.B, tmid))
					{
						tbad = tmid;
					}
					else
					{
						tsafe = tmid;
					}
				}
				CellUnmoved[CellIdx] = false;
				for (FIndex2i IDs : Update.IDs)
				{
					FVector3d OldPos = CellMeshes[IDs.A].AugMesh.GetVertex(IDs.B);
					FVector3d NewPos;
					if (IDs.A == CellIdx)
					{
						NewPos = InterpVert(IDs.B, tsafe);
						Mesh.SetVertex(IDs.B, NewPos);
					}
					else
					{
						CellUnmoved[IDs.A] = false;
						FDynamicMesh3& OtherMesh = CellMeshes[IDs.A].AugMesh;
						TDynamicMeshVertexAttribute<double, 3>* OtherOriginalPosns =
							static_cast<TDynamicMeshVertexAttribute<double, 3>*>(OtherMesh.Attributes()->GetAttachedAttribute(OriginalPositionAttribute));
						FVector3d OrigPos;
						OtherOriginalPosns->GetValue(IDs.B, OrigPos);
						NewPos = UE::Geometry::Lerp(OrigPos, OldPos, tsafe);
						OtherMesh.SetVertex(IDs.B, NewPos);
					}
					MeshesVertices.UpdatePoint(IDs, OldPos, NewPos);
				}
			}
		}
	}

	// clear "original position" attribute now that we have removed self-intersections
	// and set unset vertex colors
	for (int CellIdx = 0; CellIdx < NumCells; CellIdx++)
	{
		FCellInfo& CellInfo = CellMeshes[CellIdx];
		FDynamicMesh3& Mesh = CellInfo.AugMesh;
		Mesh.Attributes()->RemoveAttribute(OriginalPositionAttribute);
		for (int VID : Mesh.VertexIndicesItr())
		{
			AugmentedDynamicMesh::SetVertexColor(Mesh, VID, AugmentedDynamicMesh::UnsetVertexColor);
		}
	}

	// recompute UVs using new positions after noise was applied + fixed
	TArray<FVector2f> PlaneMinUVs; PlaneMinUVs.Init(FVector2f(FMathf::MaxReal, FMathf::MaxReal), Cells.Planes.Num());
	TArray<FFrame3d> PlaneFrames; PlaneFrames.Reserve(Cells.Planes.Num());
	for (int PlaneIdx = 0; PlaneIdx < Cells.Planes.Num(); PlaneIdx++)
	{
		PlaneFrames.Add(AxisAlignedFrame(Cells.Planes[PlaneIdx]));
	}
	// first pass to compute min UV for each plane
	for (FCellInfo& CellInfo : CellMeshes)
	{
		FDynamicMesh3& Mesh = CellInfo.AugMesh;
		FDynamicMeshMaterialAttribute* MaterialIDs = Mesh.Attributes()->GetMaterialID();

		for (int TID : Mesh.TriangleIndicesItr())
		{
			int PlaneIdx = MaterialToPlane(MaterialIDs->GetValue(TID));
			if (PlaneIdx > -1)
			{
				FIndex3i Tri = Mesh.GetTriangle(TID);
				for (int Idx = 0; Idx < 3; Idx++)
				{
					FVector2f UV = FVector2f(PlaneFrames[PlaneIdx].ToPlaneUV(Mesh.GetVertex(Tri[Idx])));
					FVector2f& MinUV = PlaneMinUVs[PlaneIdx];
					MinUV.X = FMathf::Min(UV.X, MinUV.X);
					MinUV.Y = FMathf::Min(UV.Y, MinUV.Y);
				}
			}
		}
	}
	// second pass to actually set UVs
	for (FCellInfo& CellInfo : CellMeshes)
	{
		FDynamicMesh3& Mesh = CellInfo.AugMesh;
		FDynamicMeshMaterialAttribute* MaterialIDs = Mesh.Attributes()->GetMaterialID();

		for (int TID : Mesh.TriangleIndicesItr())
		{
			int PlaneIdx = MaterialToPlane(MaterialIDs->GetValue(TID));
			if (PlaneIdx > -1)
			{
				FIndex3i Tri = Mesh.GetTriangle(TID);
				for (int Idx = 0; Idx < 3; Idx++)
				{
					FVector2f UV = ((FVector2f)PlaneFrames[PlaneIdx].ToPlaneUV(Mesh.GetVertex(Tri[Idx])) - PlaneMinUVs[PlaneIdx]) * static_cast<float>(GlobalUVScale);
					for (int UVLayerIdx = 0; UVLayerIdx < NumUVLayers; UVLayerIdx++)
					{
						AugmentedDynamicMesh::SetUV(Mesh, Tri[Idx], UV, UVLayerIdx);
					}
				}
			}
		}
	}
}


void FCellMeshes::CreateMeshesForSinglePlane(const FPlanarCells& Cells, const FAxisAlignedBox3d& DomainBounds, bool bNoise, double GlobalUVScale, double Grout, bool bOnlyGrout)
{
	bool bHasGrout = Grout > 0;

	int MID = PlaneToMaterial(0);
	FPlane Plane = Cells.Planes[0];

	FFrame3d PlaneFrame = AxisAlignedFrame(Plane);
	FInterval1d ZRange;
	FAxisAlignedBox2d XYRange;
	for (int CornerIdx = 0; CornerIdx < 8; CornerIdx++)
	{
		FVector3d Corner = DomainBounds.GetCorner(CornerIdx);
		XYRange.Contain(PlaneFrame.ToPlaneUV(Corner));
		ZRange.Contain(Plane.PlaneDot(FVector(Corner)));
	}
	//if (FMathd::SignNonZero(ZRange.Min) == FMathd::SignNonZero(ZRange.Max))
	//{
	//	// TODO: early out for plane that doesn't even intersect the domain bounding box?
	//}

	FDynamicMesh3 PlaneMesh(true, true, false, false);
	AugmentedDynamicMesh::EnableUVChannels(PlaneMesh, NumUVLayers);
	FVertexInfo PlaneVertInfo;
	PlaneVertInfo.bHaveC = true;
	PlaneVertInfo.bHaveN = true;
	PlaneVertInfo.Normal = -FVector3f(Plane.GetNormal());

	for (int CornerIdx = 0; CornerIdx < 4; CornerIdx++)
	{
		PlaneVertInfo.Position = PlaneFrame.FromPlaneUV(XYRange.GetCorner(CornerIdx));
		FVector2f UV = FVector2f(XYRange.GetCorner(CornerIdx) - XYRange.Min) * static_cast<float>(GlobalUVScale);
		int VID = PlaneMesh.AppendVertex(PlaneVertInfo);
		AugmentedDynamicMesh::SetAllUV(PlaneMesh, VID, UV, NumUVLayers);
	}
	PlaneMesh.AppendTriangle(0, 1, 2);
	PlaneMesh.AppendTriangle(0, 2, 3);

	if (bNoise)
	{
		double Spacing = GetSafeNoiseSpacing(static_cast<float>(XYRange.Area()), Cells.InternalSurfaceMaterials.NoiseSettings->PointSpacing);
		RemeshForNoise(PlaneMesh, EEdgeRefineFlags::SplitsOnly, Spacing);
		ApplyNoise(PlaneMesh, PlaneFrame.GetAxis(2), Cells.InternalSurfaceMaterials.NoiseSettings.GetValue(), true);
		FMeshNormals::QuickComputeVertexNormals(PlaneMesh);
	}
	TArray<int> PlaneBoundary,  // loop of vertex IDs on the boundary of PlaneMesh (starting with vertex 0)
		PlaneBoundaryCornerIndices; // indices of the corner vertices in the PlaneBoundary array
	{
		double Offset = ZRange.Max;
		FMeshBoundaryLoops Boundary(&PlaneMesh);
		checkSlow(Boundary.GetLoopCount() == 1);
		int FirstIdx;
		bool bFound = Boundary[0].Vertices.Find(0, FirstIdx);
		checkSlow(bFound);
		PlaneBoundary = Boundary[0].Vertices;
		if (FirstIdx != 0)
		{
			Algo::Rotate(PlaneBoundary, FirstIdx);
		}
		checkSlow(PlaneBoundary[0] == 0);

		PlaneBoundaryCornerIndices.Add(0);
		int FoundIndices = 1;
		for (int VIDIdx = 0; VIDIdx < PlaneBoundary.Num(); VIDIdx++)
		{
			int VID = PlaneBoundary[VIDIdx];
			if (VID == FoundIndices)
			{
				FoundIndices++;
				PlaneBoundaryCornerIndices.Add(VIDIdx);
			}
		}
	}
	FDynamicMesh3* Meshes[2];
	if (!bOnlyGrout)
	{
		for (int Side = 0; Side < 2; Side++)
		{
			Meshes[Side] = &CellMeshes[Side].AugMesh;
			*Meshes[Side] = PlaneMesh;

			double Offset = ZRange.Max;
			TArray<int> CapBoundary, CapBoundaryCornerIndices;

			if (Side == 0)
			{
				Meshes[Side]->ReverseOrientation(true);
				Offset = ZRange.Min;
			}
			PlaneVertInfo.Normal = FVector3f(Plane.GetNormal()) * (-1.0f + (float)Side * 2.0f);
			FVector3d OffsetVec = FVector3d(Plane.GetNormal()) * Offset;

			for (int CornerIdx = 0; CornerIdx < 4; CornerIdx++)
			{
				PlaneVertInfo.Position = Meshes[Side]->GetVertex(CornerIdx) + OffsetVec;
				// UVs shouldn't matter for outer box vertices because they're outside of the domain by construction ...
				CapBoundary.Add(Meshes[Side]->AppendVertex(PlaneVertInfo));
				CapBoundaryCornerIndices.Add(CornerIdx);
			}
			int NewTris[2]{
				Meshes[Side]->AppendTriangle(CapBoundary[0], CapBoundary[1], CapBoundary[2]),
				Meshes[Side]->AppendTriangle(CapBoundary[0], CapBoundary[2], CapBoundary[3])
			};
			if (Side == 1)
			{
				Meshes[Side]->ReverseTriOrientation(NewTris[0]);
				Meshes[Side]->ReverseTriOrientation(NewTris[1]);
			}
			FDynamicMeshEditor Editor(Meshes[Side]);
			FDynamicMeshEditResult ResultOut;
			Editor.StitchSparselyCorrespondedVertexLoops(PlaneBoundary, PlaneBoundaryCornerIndices, CapBoundary, CapBoundaryCornerIndices, ResultOut, Side == 0);
		}
	}
	if (bHasGrout)
	{
		int GroutIdx = bOnlyGrout ? 0 : 2;
		FDynamicMesh3* GroutMesh = &CellMeshes[GroutIdx].AugMesh;
		FVector3d GroutOffset = (FVector3d)Plane.GetNormal() * (Grout * .5);
		if (!bOnlyGrout)
		{
			for (int Side = 0; Side < 2; Side++)
			{
				// shift both sides out by Grout/2
				MeshTransforms::Translate(*Meshes[Side], GroutOffset * (-1.0 + (double)Side * 2.0));
			}
		}

		// make the center (grout) by stitching together two offset copies of PlaneMesh
		*GroutMesh = PlaneMesh;
		GroutMesh->ReverseOrientation(true);
		MeshTransforms::Translate(*GroutMesh, GroutOffset);
		FMeshIndexMappings IndexMaps;
		FDynamicMeshEditor Editor(GroutMesh);
		Editor.AppendMesh(&PlaneMesh, IndexMaps, [GroutOffset](int VID, const FVector3d& PosIn) {return PosIn - GroutOffset; });
		TArray<int> AppendPlaneBoundary; AppendPlaneBoundary.Reserve(PlaneBoundary.Num());
		TArray<int> RevBoundary = PlaneBoundary;
		Algo::Reverse(RevBoundary);
		for (int VID : RevBoundary)
		{
			AppendPlaneBoundary.Add(IndexMaps.GetNewVertex(VID));
		}
		FDynamicMeshEditResult ResultOut;
		Editor.StitchVertexLoopsMinimal(RevBoundary, AppendPlaneBoundary, ResultOut);
	}

	// fix up custom attributes and material IDs for all meshes
	for (int CellIdx = 0; CellIdx < CellMeshes.Num(); CellIdx++)
	{
		FDynamicMesh3& Mesh = CellMeshes[CellIdx].AugMesh;

		// re-enable tangents and visibility attributes, since these are lost when we set the mesh to a copy of the plane mesh
		AugmentedDynamicMesh::Augment(Mesh, NumUVLayers);

		// Set all material IDs to the one plane's corresponding material ID
		for (int TID : Mesh.TriangleIndicesItr())
		{
			Mesh.Attributes()->GetMaterialID()->SetNewValue(TID, MID);
		}
	}
}

template<typename TransformType>
void FDynamicMeshCollection::InitTemplate(const FGeometryCollection* Collection, TArrayView<const TransformType> Transforms, const TArrayView<const int32>& TransformIndices, FTransform TransformCollection, bool bSaveIsolatedVertices)
{
	GeometryCollection::UV::FConstUVLayers UVLayers = GeometryCollection::UV::FindActiveUVLayers(*Collection);
	int32 NumUVLayers = UVLayers.Num();
	Meshes.Reset();
	Bounds = FAxisAlignedBox3d::Empty();

	for (int32 TransformIdx : TransformIndices)
	{
		int32 GeometryIdx = Collection->TransformToGeometryIndex[TransformIdx];
		if (GeometryIdx == INDEX_NONE)
		{
			// only store the mesh if there is associated geometry
			continue;
		}

		FTransformSRT3d CollectionToLocal;
		if (bComponentSpaceTransforms)
		{
			CollectionToLocal = FTransformSRT3d(FTransform(Transforms[TransformIdx]) * TransformCollection);
		}
		else
		{
			CollectionToLocal = FTransformSRT3d(GeometryCollectionAlgo::GlobalMatrix(Transforms, TArrayView<const int32>(Collection->Parent.GetConstArray()), TransformIdx) * TransformCollection);
		}

		int32 AddedMeshIdx = Meshes.Add(new FMeshData(NumUVLayers));
		FMeshData& MeshData = Meshes[AddedMeshIdx];
		MeshData.TransformIndex = TransformIdx;
		MeshData.FromCollection = FTransform(CollectionToLocal);
		FDynamicMesh3& Mesh = MeshData.AugMesh;
		
		Mesh.EnableAttributes();
		Mesh.Attributes()->EnableMaterialID();

		int32 VertexStart = Collection->VertexStart[GeometryIdx];
		int32 VertexCount = Collection->VertexCount[GeometryIdx];
		int32 FaceCount = Collection->FaceCount[GeometryIdx];

		FVertexInfo VertexInfo;
		VertexInfo.bHaveC = true;
		VertexInfo.bHaveN = true;
		VertexInfo.bHaveUV = false;
		for (int32 Idx = VertexStart, N = VertexStart + VertexCount; Idx < N; Idx++)
		{
			VertexInfo.Position = CollectionToLocal.TransformPosition(FVector3d(Collection->Vertex[Idx]));
			VertexInfo.Color = FVector3f(Collection->Color[Idx]);
			VertexInfo.Normal = (FVector3f)CollectionToLocal.TransformVectorNoScale(FVector3d(Collection->Normal[Idx]));
			int VID = Mesh.AppendVertex(VertexInfo);
			AugmentedDynamicMesh::SetVertexColor(Mesh, VID, FVector4f(Collection->Color[Idx]));
			AugmentedDynamicMesh::SetTangent(Mesh, VID, VertexInfo.Normal,
				(FVector3f)CollectionToLocal.TransformVectorNoScale(FVector3d(Collection->TangentU[Idx])),
				(FVector3f)CollectionToLocal.TransformVectorNoScale(FVector3d(Collection->TangentV[Idx])));
			
			for (int32 UVLayer = 0; UVLayer < NumUVLayers; ++UVLayer)
			{
				AugmentedDynamicMesh::SetUV(Mesh, VID, UVLayers[UVLayer][Idx], UVLayer);
			}
		}
		FIntVector VertexOffset(VertexStart, VertexStart, VertexStart);
		for (int32 Idx = Collection->FaceStart[GeometryIdx], N = Collection->FaceStart[GeometryIdx] + FaceCount; Idx < N; Idx++)
		{
			if (bSkipInvisible && !Collection->Visible[Idx])
			{
				continue;
			}
			FIndex3i AddTri = FIndex3i(Collection->Indices[Idx] - VertexOffset);
			int TID = Mesh.AppendTriangle(AddTri, 0);
			if (TID == FDynamicMesh3::NonManifoldID)
			{
				// work around non-manifold triangles by copying the vertices
				FIndex3i NewTri(-1, -1, -1);
				for (int SubIdx = 0; SubIdx < 3; SubIdx++)
				{
					int NewVID = Mesh.AppendVertex(Mesh, AddTri[SubIdx]);
					int32 SrcIdx = AddTri[SubIdx] + VertexStart;
					AugmentedDynamicMesh::SetTangent(Mesh, NewVID,
						Mesh.GetVertexNormal(NewVID), // TODO: we don't actually use the vertex normal; consider removing this arg from the function entirely
						(FVector3f)CollectionToLocal.TransformVectorNoScale(FVector3d(Collection->TangentU[SrcIdx])),
						(FVector3f)CollectionToLocal.TransformVectorNoScale(FVector3d(Collection->TangentV[SrcIdx])));

					for (int32 UVLayer = 0; UVLayer < NumUVLayers; ++UVLayer)
					{
						AugmentedDynamicMesh::SetUV(Mesh, NewVID, UVLayers[UVLayer][SrcIdx], UVLayer);
					}

					NewTri[SubIdx] = NewVID;
				}
				TID = Mesh.AppendTriangle(NewTri, 0);
			}
			if (TID < 0)
			{
				continue;
			}

			if (bGenerateMeshToCollectionFaceMapping)
			{
				MeshData.AddMeshToCollectionFaceMapping(TID, Idx);
			}

			Mesh.Attributes()->GetMaterialID()->SetValue(TID, Collection->MaterialID[Idx]);
			AugmentedDynamicMesh::SetVisibility(Mesh, TID, Collection->Visible[Idx]);
			AugmentedDynamicMesh::SetInternal(Mesh, TID, Collection->Internal[Idx]);
			// note: material index doesn't need to be passed through; will be rebuilt by a call to reindex materials once the cut mesh is returned back to geometry collection format
		}

		if (!bSaveIsolatedVertices)
		{
			FDynamicMeshEditor Editor(&Mesh);
			Editor.RemoveIsolatedVertices();
		}

		Bounds.Contain(MeshData.GetCachedBounds());

		// TODO: build spatial data (add this after setting up mesh boolean path that can use it)
		//MeshData.Spatial.SetMesh(&Mesh);
	}
}


void FDynamicMeshCollection::Init(const FGeometryCollection* Collection, TArrayView<const FTransform> Transforms, const TArrayView<const int32>& TransformIndices, FTransform TransformCollection, bool bSaveIsolatedVertices)
{
	InitTemplate(Collection, Transforms, TransformIndices, TransformCollection, bSaveIsolatedVertices);
}

void FDynamicMeshCollection::Init(const FGeometryCollection* Collection, TArrayView<const FTransform3f> Transforms, const TArrayView<const int32>& TransformIndices, FTransform TransformCollection, bool bSaveIsolatedVertices)
{
	InitTemplate(Collection, Transforms, TransformIndices, TransformCollection, bSaveIsolatedVertices);
}

void FDynamicMeshCollection::SetGeometryVisibility(FGeometryCollection* Collection, const TArray<int32>& GeometryIndices, bool bVisible)
{
	TManagedArray<bool>& Visible = Collection->Visible;
	for (int32 GeoIdx : GeometryIndices)
	{
		int32 FaceStart = Collection->FaceStart[GeoIdx];
		int32 FaceEnd = FaceStart + Collection->FaceCount[GeoIdx];
		for (int32 FaceIdx = FaceStart; FaceIdx < FaceEnd; FaceIdx++)
		{
			Visible[FaceIdx] = bVisible;
		}
	}
}


int32 FDynamicMeshCollection::CutWithMultiplePlanes(
	const TArrayView<const FPlane>& Planes,
	double Grout,
	double CollisionSampleSpacing,
	const FIslandSplitSettings& IslandSplitSettings,
	int32 RandomSeed,
	FGeometryCollection* Collection,
	FInternalSurfaceMaterials& InternalSurfaceMaterials,
	bool bSetDefaultInternalMaterialsFromCollection,
	FProgressCancel* Progress
)
{
	bool bHasGrout = Grout > 0;

	int32 NumUVLayers = Collection->NumUVLayers();

	FRandomStream RandomStream(RandomSeed);

	const float PerPlaneWorkFrac = 1.0f / Planes.Num();

	if (bHasGrout)
	{
		// For multi-plane cuts with grout specifically, the easiest path seems to be:
		// 1. Build the "grout" section of each plane
		// 2. Take the union of all those grout sections as the grout mesh
		// 3. Use the generic CutWithCellMeshes path, where that grout mesh is both the inner and outside cell mesh
		//    (Note the outside cell mesh is subtracted, not intersected)
		//    (Note this relies on island splitting to separate all the pieces afterwards.)
		FCellMeshes GroutCells(NumUVLayers, RandomStream);
		GroutCells.SetNumCells(2);
		FDynamicMesh3& GroutMesh = GroutCells.CellMeshes[0].AugMesh;
		FDynamicMeshEditor GroutAppender(&GroutMesh);
		FMeshIndexMappings IndexMaps;
		for (int32 PlaneIdx = 0; PlaneIdx < Planes.Num(); PlaneIdx++)
		{
			FProgressCancel::FProgressScope MakeMeshScope(Progress, .1f * PerPlaneWorkFrac);
			FPlanarCells PlaneCells(Planes[PlaneIdx]);
			PlaneCells.InternalSurfaceMaterials = InternalSurfaceMaterials;
			FCellMeshes PlaneGroutMesh(NumUVLayers, RandomStream);
			PlaneGroutMesh.MakeOnlyPlanarGroutCell(PlaneCells, Bounds, Grout);
			GroutAppender.AppendMesh(&PlaneGroutMesh.CellMeshes[0].AugMesh, IndexMaps);
			if (Progress && Progress->Cancelled())
			{
				return INDEX_NONE;
			}
		}

		FProgressCancel::FProgressScope GroutUnionScope(Progress, .3f);
		FMeshSelfUnion GroutUnion(&GroutMesh);
		GroutUnion.bSimplifyAlongNewEdges = true;
		GroutUnion.bWeldSharedEdges = false;
		GroutUnion.Compute();
		GroutUnionScope.Done();
		if (Progress && Progress->Cancelled())
		{
			return INDEX_NONE;
		}
		
		FProgressCancel::FProgressScope GroutCutScope = FProgressCancel::CreateScopeTo(Progress, 1);
		// first mesh is the same as the second mesh, but will be subtracted b/c it's the "outside cell"
		GroutCells.CellMeshes[1].AugMesh = GroutMesh;
		GroutCells.OutsideCellIndex = 1;

		TArray<TPair<int32, int32>> CellConnectivity;
		CellConnectivity.Add(TPair<int32, int32>(0, -1));

		return CutWithCellMeshes(InternalSurfaceMaterials, CellConnectivity, GroutCells, IslandSplitSettings, Collection, bSetDefaultInternalMaterialsFromCollection, CollisionSampleSpacing);
	}

	TArray<TUniquePtr<FMeshData>> ToCut;
	// copy initial surfaces
	for (FMeshData& MeshData : Meshes)
	{
		ToCut.Add(MakeUnique<FMeshData>(MeshData));
	}
	for (int32 PlaneIdx = 0; PlaneIdx < Planes.Num(); PlaneIdx++)
	{
		if (Progress && Progress->Cancelled())
		{
			return INDEX_NONE;
		}
		FProgressCancel::FProgressScope OnePlaneScope(Progress, PerPlaneWorkFrac);
		FPlanarCells PlaneCells(Planes[PlaneIdx]);
		PlaneCells.InternalSurfaceMaterials = InternalSurfaceMaterials;
		double OnePercentExtend = Bounds.MaxDim() * .01;
		FCellMeshes CellMeshes(NumUVLayers, RandomStream, PlaneCells, Bounds, 0, OnePercentExtend, false);

		// TODO: we could do these cuts in parallel (will takes some rework of how results are added to the ToCut array)
		for (int32 ToCutIdx = 0, ToCutNum = ToCut.Num(); ToCutIdx < ToCutNum; ToCutIdx++)
		{
			FMeshData& Surface = *ToCut[ToCutIdx];
			int32 TransformIndex = Surface.TransformIndex;
			FTransform FromCollection = Surface.FromCollection;

			FAxisAlignedBox3d Box = Surface.GetCachedBounds();
			if (InternalSurfaceMaterials.NoiseSettings.IsSet())
			{
				Box.Expand(InternalSurfaceMaterials.NoiseSettings->Amplitude);
			}
			if (!FMath::PlaneAABBIntersection(Planes[PlaneIdx], FBox(Box)))
			{
				continue;
			}

			TArray<TUniquePtr<FMeshData>> BoolResults;
			for (int ResultIdx = 0; ResultIdx < 2; ResultIdx++)
			{
				BoolResults.Add(MakeUnique<FMeshData>(NumUVLayers));
				BoolResults[ResultIdx]->TransformIndex = TransformIndex;
				BoolResults[ResultIdx]->FromCollection = FromCollection;
			}
			check(CellMeshes.CellMeshes.Num() == 2);
			bool bKeepResults = true;
			for (int32 CellIdx = 0; CellIdx < 2; CellIdx++)
			{
				FCellMeshes::FCellInfo& Cell = CellMeshes.CellMeshes[CellIdx];

				FMeshBoolean::EBooleanOp Op = FMeshBoolean::EBooleanOp::Intersect;
				if (CellIdx == CellMeshes.OutsideCellIndex)
				{
					Op = FMeshBoolean::EBooleanOp::Difference;
				}
				FMeshBoolean Boolean(&Surface.AugMesh, &Cell.AugMesh, &BoolResults[CellIdx]->AugMesh, Op);
				Boolean.bSimplifyAlongNewEdges = true;
				Boolean.PreserveUVsOnlyForMesh = 0; // slight warping of the autogenerated cell UVs generally doesn't matter
				Boolean.bWeldSharedEdges = false;
				Boolean.bTrackAllNewEdges = true;
				if (!Boolean.Compute())
				{
					// TODO: do something about failure cases?  e.g. try auto-filling small holes?
					// note: failure cases won't be detected at all unless we weld edges,
					//       which will require re-working how tangents are carried through
				}
				if (BoolResults[CellIdx]->AugMesh.TriangleCount() == 0)
				{
					bKeepResults = false;
					break;
				}
				constexpr bool bFillHoles = false; // TODO: consider enabling hole filler optionally, or improving the hole filler and enabling it all the time
				if (bFillHoles)
				{
					AugmentedDynamicMesh::FillHoles(BoolResults[CellIdx]->AugMesh, Boolean.AllNewEdges);
				}
			}

			if (bKeepResults)
			{
				ToCut[ToCutIdx] = MoveTemp(BoolResults[0]);
				int32 NewIdx = ToCut.Add(MoveTemp(BoolResults[1]));
				// indices of all boolean result meshes (may be more than two due to splitting disconnected components)
				TArray<int32, TInlineAllocator<4>> ResultIndices = { ToCutIdx, NewIdx };
				// corresponding parent indices for each result mesh
				TArray<int32, TInlineAllocator<4>> ParentIndices = { 0, 1 };
				TArray<TUniquePtr<FDynamicMesh3>> SplitMeshes;
				for (int UnsplitIdx = 0; UnsplitIdx < 2; UnsplitIdx++)
				{
					if (IslandSplitSettings.bSplitIslands && SplitIslands(ToCut[ResultIndices[UnsplitIdx]]->AugMesh, SplitMeshes, IslandSplitSettings))
					{
						// TODO: Add a SetMesh(FDynamicMesh3&&) method so we can MoveTemp here
						ToCut[ResultIndices[UnsplitIdx]]->SetMesh(*SplitMeshes[0]);
						for (int32 Idx = 1; Idx < SplitMeshes.Num(); Idx++)
						{
							const FDynamicMesh3& Mesh = *SplitMeshes[Idx];
							// TODO: Add a FMeshData constructor to let us MoveTemp the mesh here
							ResultIndices.Add(ToCut.Add(MakeUnique<FMeshData>(Mesh, TransformIndex, FromCollection)));
							ParentIndices.Add(UnsplitIdx);
						}
					}
				}

			} // iteration over meshes to cut
		} // iteration over cutting planes
	}

	TMultiMap<int32, int32> ParentTransformToChildren;
	for (int32 ToCutIdx = 0; ToCutIdx < ToCut.Num(); ToCutIdx++)
	{
		ParentTransformToChildren.Add(ToCut[ToCutIdx]->TransformIndex, ToCutIdx);
	}

	TArray<int32> ToCutIdxToGeometryIdx;  ToCutIdxToGeometryIdx.Init(-1, ToCut.Num());
	TArray<int32> ToCutIndices;
	int32 FirstCreatedIndex = -1;
	TArray<int32> GeometryForRemoval;
	for (FMeshData& MeshData : Meshes)
	{
		int32 GeometryIdx = Collection->TransformToGeometryIndex[MeshData.TransformIndex];
		int32 InternalMaterialID = bSetDefaultInternalMaterialsFromCollection ? InternalSurfaceMaterials.GetDefaultMaterialIDForGeometry(*Collection, GeometryIdx) : InternalSurfaceMaterials.GlobalMaterialID;
		ToCutIndices.Reset();
		ParentTransformToChildren.MultiFind(MeshData.TransformIndex, ToCutIndices);

		// if there's only one mesh here, i.e. it didn't get cut at all
		if (ToCutIndices.Num() <= 1)
		{
			continue;
		}

		// remove old parent geometry
		GeometryForRemoval.Add(GeometryIdx);

		// add newly created geometry as children
		int32 SubPartIdx = 0;
		for (int32 ToCutIdx : ToCutIndices)
		{
			FDynamicMesh3& Mesh = ToCut[ToCutIdx]->AugMesh;

			FString BoneName = GetBoneName(*Collection, ToCut[ToCutIdx]->TransformIndex, SubPartIdx++);
			int32 CreatedGeometryIdx = AppendToCollection(ToCut[ToCutIdx]->FromCollection, Mesh, CollisionSampleSpacing, ToCut[ToCutIdx]->TransformIndex, BoneName, *Collection, InternalMaterialID);
			ToCutIdxToGeometryIdx[ToCutIdx] = CreatedGeometryIdx;
			if (FirstCreatedIndex == -1)
			{
				FirstCreatedIndex = CreatedGeometryIdx;
			}
		}
	}

	if (Progress && Progress->Cancelled())
	{
		return INDEX_NONE;
	}

	SetUnsetColors(Collection, FirstCreatedIndex);

	int32 NewFirstIndex = FirstCreatedIndex;

	constexpr bool bRemoveOldGeometry = false; // if false, we just hide the geometry that we've replaced by fractured child geometry, rather than remove it
	if (GeometryForRemoval.Num() > 0)
	{
		if (bRemoveOldGeometry)
		{
			GeometryForRemoval.Sort();
			FManagedArrayCollection::FProcessingParameters ProcessingParams;
#if !UE_BUILD_DEBUG
			ProcessingParams.bDoValidation = false;
#endif
			Collection->RemoveElements(FGeometryCollection::GeometryGroup, GeometryForRemoval, ProcessingParams);
			NewFirstIndex -= GeometryForRemoval.Num();
		}
		else
		{
			SetGeometryVisibility(Collection, GeometryForRemoval, false);
		}
	}

	return NewFirstIndex;
}


int32 FDynamicMeshCollection::SplitAllIslands(FGeometryCollection* Collection, double CollisionSampleSpacing, const FIslandSplitSettings& IslandSplitSettings)
{
	int32 FirstIdx = -1;
	TArray<int32> GeometryForRemoval;

	for (FMeshData& Surface : Meshes)
	{
		int32 SrcGeometryIdx = Collection->TransformToGeometryIndex[Surface.TransformIndex];
		
		int32 CreatedGeometryIdx = -1;
		TArray<TUniquePtr<FDynamicMesh3>> Islands;
		if (SplitIslands(Surface.AugMesh, Islands, IslandSplitSettings))
		{
			for (int32 i = 0; i < Islands.Num(); i++)
			{
				FDynamicMesh3& Island = *Islands[i];
				FString BoneName = GetBoneName(*Collection, Surface.TransformIndex, i);
				constexpr int32 InternalMaterialID = 0; // Note: there won't be new internal faces (to assign materials) from a split, so this value won't affect the output here
				CreatedGeometryIdx = AppendToCollection(Surface.FromCollection, Island, CollisionSampleSpacing, Surface.TransformIndex, BoneName, *Collection, InternalMaterialID);

				if (FirstIdx == -1)
				{
					FirstIdx = CreatedGeometryIdx;
				}
			}

			GeometryForRemoval.Add(SrcGeometryIdx);
		}
	}

	int32 NewFirstIdx = FirstIdx;

	// remove or hide superfluous geometry
	constexpr bool bRemoveOldGeometry = false; // if false, we just hide the geometry that we've replaced by fractured child geometry, rather than remove it
	if (GeometryForRemoval.Num() > 0)
	{
		if (bRemoveOldGeometry)
		{
			GeometryForRemoval.Sort();
			FManagedArrayCollection::FProcessingParameters ProcessingParams;
#if !UE_BUILD_DEBUG
			ProcessingParams.bDoValidation = false;
#endif
			Collection->RemoveElements(FGeometryCollection::GeometryGroup, GeometryForRemoval, ProcessingParams);
			NewFirstIdx -= GeometryForRemoval.Num();
		}
		else
		{
			SetGeometryVisibility(Collection, GeometryForRemoval, false);
		}
	}


	return NewFirstIdx;
}



int32 FDynamicMeshCollection::CutWithCellMeshes(const FInternalSurfaceMaterials& InternalSurfaceMaterials, const TArray<TPair<int32, int32>>& CellConnectivity, FCellMeshes& CellMeshes, const FIslandSplitSettings& IslandSplitSettings, FGeometryCollection* Collection, bool bSetDefaultInternalMaterialsFromCollection, double CollisionSampleSpacing)
{
	// TODO: should we do these cuts in parallel, and the appends sequentially below?
	int32 FirstIdx = -1;
	int BadCount = 0;
	TArray<int32> GeometryForRemoval;
	TArray<FAxisAlignedBox3d> CellBounds; CellBounds.Reserve(CellMeshes.CellMeshes.Num());
	for (int CellIdx = 0; CellIdx < CellMeshes.CellMeshes.Num(); CellIdx++)
	{
		CellBounds.Add(CellMeshes.CellMeshes[CellIdx].AugMesh.GetBounds(true));
	}
	for (FMeshData& Surface : Meshes)
	{
		int32 GeometryIdx = Collection->TransformToGeometryIndex[Surface.TransformIndex];
		TArray<TUniquePtr<FDynamicMesh3>> BooleanResults;  BooleanResults.SetNum(CellMeshes.CellMeshes.Num());
		FAxisAlignedBox3d SurfaceBounds = Surface.GetCachedBounds();

		ParallelFor(CellMeshes.CellMeshes.Num(), [&BooleanResults, &CellMeshes, &CellBounds, &Surface, &SurfaceBounds](int32 CellIdx)
			{
				FCellMeshes::FCellInfo& Cell = CellMeshes.CellMeshes[CellIdx];
				if (CellBounds[CellIdx].Intersects(SurfaceBounds))
				{
					BooleanResults[CellIdx] = MakeUnique<FDynamicMesh3>();
					FDynamicMesh3& AugBoolResult = *BooleanResults[CellIdx];
					FMeshBoolean::EBooleanOp Op = FMeshBoolean::EBooleanOp::Intersect;
					if (CellIdx == CellMeshes.OutsideCellIndex)
					{
						Op = FMeshBoolean::EBooleanOp::Difference;
					}
					FMeshBoolean Boolean(&Surface.AugMesh, &Cell.AugMesh, &AugBoolResult, Op);
					Boolean.bSimplifyAlongNewEdges = true;
					Boolean.PreserveUVsOnlyForMesh = 0; // slight warping of the autogenerated cell UVs generally doesn't matter
					Boolean.bWeldSharedEdges = false;
					Boolean.bTrackAllNewEdges = true;
					if (!Boolean.Compute())
					{
						// TODO: do something about failure cases?  e.g. try auto-filling small holes?
						// note: failure cases won't be detected at all unless we weld edges,
						//       which will require re-working how tangents are carried through
					}
					constexpr bool bFillHoles = false; // TODO: consider enabling hole filler optionally, or improving the hole filler and enabling it all the time
					if (bFillHoles)
					{
						AugmentedDynamicMesh::FillHoles(AugBoolResult, Boolean.AllNewEdges);
					}
				}
			}, EParallelForFlags::None);

		int32 NonEmptyResults = 0;
		for (const TUniquePtr<FDynamicMesh3>& AugBoolResult : BooleanResults)
		{
			if (AugBoolResult.IsValid() && AugBoolResult->TriangleCount() > 0)
			{
				NonEmptyResults++;
			}
		}

		if (NonEmptyResults > 1) // only write to geometry collection if more than one result was non-empty
		{
			TSet<int32> PlanesInOutput;
			TMultiMap<int32, int32> CellToGeometry;
			TMap<int32, int32> GeometryToResultMesh;
			int32 SubPartIndex = 0;
			int32 InternalMaterialID = bSetDefaultInternalMaterialsFromCollection ? InternalSurfaceMaterials.GetDefaultMaterialIDForGeometry(*Collection, GeometryIdx) : InternalSurfaceMaterials.GlobalMaterialID;

			for (int32 CellIdx = 0; CellIdx < CellMeshes.CellMeshes.Num(); CellIdx++)
			{
				if (BooleanResults[CellIdx].IsValid() && BooleanResults[CellIdx]->TriangleCount() > 0)
				{
					FDynamicMesh3& AugBoolResult = *BooleanResults[CellIdx];
					for (int TID : AugBoolResult.TriangleIndicesItr())
					{
						int MID = AugBoolResult.Attributes()->GetMaterialID()->GetValue(TID);
						int32 PlaneIdx = CellMeshes.MaterialToPlane(MID);
						if (PlaneIdx >= 0)
						{
							PlanesInOutput.Add(PlaneIdx);
						}
					}
					int32 CreatedGeometryIdx = -1;
					TArray<TUniquePtr<FDynamicMesh3>> Islands;
					if (IslandSplitSettings.bSplitIslands && SplitIslands(AugBoolResult, Islands, IslandSplitSettings))
					{
						for (int32 i = 0; i < Islands.Num(); i++)
						{
							FDynamicMesh3& Island = *Islands[i];
							FString BoneName = GetBoneName(*Collection, Surface.TransformIndex, SubPartIndex++);
							CreatedGeometryIdx = AppendToCollection(Surface.FromCollection, Island, CollisionSampleSpacing, Surface.TransformIndex, BoneName, *Collection, InternalMaterialID);
							CellToGeometry.Add(CellIdx, CreatedGeometryIdx);
							if (i > 0)
							{
								GeometryToResultMesh.Add(CreatedGeometryIdx, BooleanResults.Add(MakeUnique<FDynamicMesh3>(Island)));
							}
							else
							{
								*BooleanResults[CellIdx] = Island;
								GeometryToResultMesh.Add(CreatedGeometryIdx, CellIdx);
							}

							if(FirstIdx == -1)
							{
								FirstIdx = CreatedGeometryIdx;
							}
						}
					}
					else
					{
						FString BoneName = GetBoneName(*Collection, Surface.TransformIndex, SubPartIndex++);
						CreatedGeometryIdx = AppendToCollection(Surface.FromCollection, AugBoolResult, CollisionSampleSpacing, Surface.TransformIndex, BoneName, *Collection, InternalMaterialID);
						CellToGeometry.Add(CellIdx, CreatedGeometryIdx);
						GeometryToResultMesh.Add(CreatedGeometryIdx, CellIdx);

						if(FirstIdx == -1)
						{
							FirstIdx = CreatedGeometryIdx;
						}
					}
				}
			}
			
			// remove old geom
			GeometryForRemoval.Add(GeometryIdx);
		}
	}

	SetUnsetColors(Collection, FirstIdx);

	int32 NewFirstIdx = FirstIdx;

	// remove or hide superfluous geometry
	constexpr bool bRemoveOldGeometry = false; // if false, we just hide the geometry that we've replaced by fractured child geometry, rather than remove it
	if (GeometryForRemoval.Num() > 0)
	{
		if (bRemoveOldGeometry)
		{
			GeometryForRemoval.Sort();
			FManagedArrayCollection::FProcessingParameters ProcessingParams;
#if !UE_BUILD_DEBUG
			ProcessingParams.bDoValidation = false;
#endif
			Collection->RemoveElements(FGeometryCollection::GeometryGroup, GeometryForRemoval, ProcessingParams);
			NewFirstIdx -= GeometryForRemoval.Num();
		}
		else
		{
			SetGeometryVisibility(Collection, GeometryForRemoval, false);
		}
	}

	return NewFirstIdx;
}


void FDynamicMeshCollection::FillVertexHash(const FDynamicMesh3& Mesh, TPointHashGrid3d<int>& VertHash)
{
	for (int VID : Mesh.VertexIndicesItr())
	{
		FVector3d V = Mesh.GetVertex(VID);
		VertHash.InsertPointUnsafe(VID, V);
	}
}

bool FDynamicMeshCollection::IsNeighboring(
	UE::Geometry::FDynamicMesh3& MeshA, const UE::Geometry::TPointHashGrid3d<int>& VertHashA, const UE::Geometry::FAxisAlignedBox3d& BoundsA,
	UE::Geometry::FDynamicMesh3& MeshB, const UE::Geometry::TPointHashGrid3d<int>& VertHashB, const UE::Geometry::FAxisAlignedBox3d& BoundsB)
{
	UE::Geometry::FDynamicMesh3* Mesh[2]{ &MeshA, &MeshB };
	const UE::Geometry::TPointHashGrid3d<int>* VertHash[2]{ &VertHashA, &VertHashB };

	if (!ensure(Mesh[0] && Mesh[1] && VertHash[0] && VertHash[1]))
	{
		return false;
	}
	if (!BoundsA.Intersects(BoundsB))
	{
		return false;
	}

	int A = 0, B = 1;
	if (Mesh[0]->VertexCount() > Mesh[1]->VertexCount())
	{
		Swap(A, B);
	}
	FDynamicMesh3& RefMesh = *Mesh[B];
	for (const FVector3d& V : Mesh[A]->VerticesItr())
	{
		TPair<int, double> Nearest = VertHash[B]->FindNearestInRadius(V, FMathd::ZeroTolerance * 10, [&RefMesh, &V](int VID)
			{
				return DistanceSquared(RefMesh.GetVertex(VID), V);
			});
		if (Nearest.Key != -1)
		{
			return true;
		}
	}
	return false;
}

// Split mesh into connected components, including implicit connections by co-located vertices
bool FDynamicMeshCollection::SplitIslands(FDynamicMesh3& Source, TArray<TUniquePtr<FDynamicMesh3>>& SeparatedMeshes, const FIslandSplitSettings& Settings)
{
	SeparatedMeshes.Reset();

	FVertexConnectedComponents VertComponents;
	VertComponents.Init(Source);
	VertComponents.ConnectTriangles(Source);

	if (Settings.CloseVertexDistance > 0.)
	{
		VertComponents.ConnectCloseVertices(Source, Settings.CloseVertexDistance);
	}

	if (Settings.VertexToSurfaceBridgeDistance > 0.)
	{
		TMeshAABBTree3<FDynamicMesh3> Spatial(&Source);
		VertComponents.ConnectVerticesToNearestDifferentComponent(Spatial, Settings.VertexToSurfaceBridgeDistance);
	}

	bool bWasSplit = FDynamicMeshEditor::SplitMesh(&Source, SeparatedMeshes, false, [&Source, &VertComponents](int TID)
		{
			return VertComponents.GetComponent(Source.GetTriangle(TID).A);
		});

	if (bWasSplit)
	{
		// disconnected components that are contained inside other components need to be re-merged
		TMeshSpatialSort<FDynamicMesh3> SpatialSort;
		SpatialSort.NestingMethod = TMeshSpatialSort<FDynamicMesh3>::ENestingMethod::InLargestParent;
		SpatialSort.bOnlyNestNegativeVolumes = false;
		SpatialSort.bOnlyParentPostiveVolumes = true;
		SpatialSort.Compute(SeparatedMeshes);
		TArray<bool> KeepMeshes; KeepMeshes.Init(true, SeparatedMeshes.Num());
		for (TMeshSpatialSort<FDynamicMesh3>::FMeshNesting& Nest : SpatialSort.Nests)
		{
			FDynamicMeshEditor Editor(SeparatedMeshes[Nest.OuterIndex].Get());
			FMeshIndexMappings Mappings;
			for (int Inner : Nest.InnerIndices)
			{
				Editor.AppendMesh(SeparatedMeshes[Inner].Get(), Mappings);
				KeepMeshes[Inner] = false;
			}
		}
		for (int Idx = 0; Idx < SeparatedMeshes.Num(); Idx++)
		{
			if (!KeepMeshes[Idx])
			{
				SeparatedMeshes.RemoveAtSwap(Idx, EAllowShrinking::No);
				KeepMeshes.RemoveAtSwap(Idx, EAllowShrinking::No);
				Idx--;
			}
		}
	}
	return bWasSplit;
}

	
void FDynamicMeshCollection::AddCollisionSamples(double CollisionSampleSpacing)
{
	for (int32 MeshIdx = 0; MeshIdx < Meshes.Num(); MeshIdx++)
	{
		AugmentedDynamicMesh::AddCollisionSamplesPerComponent(Meshes[MeshIdx].AugMesh, CollisionSampleSpacing);
	}
}


// Update all geometry in a GeometryCollection w/ the meshes in the MeshCollection
// Resizes the GeometryCollection as needed
bool FDynamicMeshCollection::UpdateAllCollections(FGeometryCollection& Collection)
{
	bool bAllSucceeded = true;

	int32 NumGeometry = Collection.NumElements(FGeometryCollection::GeometryGroup);
	TArray<int32> NewFaceCounts, NewVertexCounts;
	NewFaceCounts.SetNumUninitialized(NumGeometry);
	NewVertexCounts.SetNumUninitialized(NumGeometry);
	for (int32 GeomIdx = 0; GeomIdx < Collection.FaceCount.Num(); GeomIdx++)
	{
		NewFaceCounts[GeomIdx] = Collection.FaceCount[GeomIdx];
		NewVertexCounts[GeomIdx] = Collection.VertexCount[GeomIdx];
	}
	for (int MeshIdx = 0; MeshIdx < Meshes.Num(); MeshIdx++)
	{
		FDynamicMeshCollection::FMeshData& MeshData = Meshes[MeshIdx];
		int32 GeomIdx = Collection.TransformToGeometryIndex[MeshData.TransformIndex];
		NewFaceCounts[GeomIdx] = MeshData.AugMesh.TriangleCount();
		NewVertexCounts[GeomIdx] = MeshData.AugMesh.VertexCount();
	}
	bool bDoValidation = false;
#if UE_BUILD_DEBUG
	bDoValidation = true;
#endif
	GeometryCollectionAlgo::ResizeGeometries(&Collection, NewFaceCounts, NewVertexCounts, bDoValidation);

	for (int MeshIdx = 0; MeshIdx < Meshes.Num(); MeshIdx++)
	{
		FDynamicMeshCollection::FMeshData& MeshData = Meshes[MeshIdx];
		FDynamicMesh3& Mesh = MeshData.AugMesh;
		int32 GeometryIdx = Collection.TransformToGeometryIndex[MeshData.TransformIndex];
		bool bSucceeded = UpdateCollection(MeshData.FromCollection, Mesh, GeometryIdx, Collection, -1);
		bAllSucceeded &= bSucceeded;
	}

	SetUnsetColors(&Collection, 0, false);

	return bAllSucceeded;
}


// Update an existing geometry in a collection w/ a new mesh (w/ the same number of faces and vertices!)
bool FDynamicMeshCollection::UpdateCollection(const FTransform& FromCollection, FDynamicMesh3& Mesh, int32 GeometryIdx, FGeometryCollection& Output, int32 InternalMaterialID)
{
	if (!Mesh.IsCompact())
	{
		Mesh.CompactInPlace(nullptr);
	}

	int32 OldVertexCount = Output.VertexCount[GeometryIdx];
	int32 OldTriangleCount = Output.FaceCount[GeometryIdx];

	int32 UVLayerCount = AugmentedDynamicMesh::NumEnabledUVChannels(Mesh);
	Output.SetNumUVLayers(UVLayerCount);
	GeometryCollection::UV::FUVLayers OutputUVLayers = GeometryCollection::UV::FindActiveUVLayers(Output);

	int32 NewVertexCount = Mesh.VertexCount();
	int32 NewTriangleCount = Mesh.TriangleCount();

	if (!ensure(OldVertexCount == NewVertexCount) || !ensure(OldTriangleCount == NewTriangleCount))
	{
		return false;
	}

	int32 VerticesStart = Output.VertexStart[GeometryIdx];
	int32 FacesStart = Output.FaceStart[GeometryIdx];
	int32 TransformIdx = Output.TransformIndex[GeometryIdx];

	for (int32 VID = 0; VID < Mesh.MaxVertexID(); VID++)
	{
		checkSlow(Mesh.IsVertex(VID)); // mesh is compact
		int32 CopyToIdx = VerticesStart + VID;
		Output.Vertex[CopyToIdx] = (FVector3f)FromCollection.InverseTransformPosition(FVector(Mesh.GetVertex(VID)));
		Output.Normal[CopyToIdx] = (FVector3f)FromCollection.InverseTransformVectorNoScale(FVector(Mesh.GetVertexNormal(VID)));
		
		for (int32 UVLayer = 0; UVLayer < UVLayerCount; ++UVLayer)
		{
			FVector2f UV;
			AugmentedDynamicMesh::GetUV(Mesh, VID, UV, UVLayer);
			OutputUVLayers[UVLayer][CopyToIdx] = UV;
		}
		
		FVector3f TangentU, TangentV;
		AugmentedDynamicMesh::GetTangent(Mesh, VID, TangentU, TangentV);
		Output.TangentU[CopyToIdx] = (FVector3f)FromCollection.InverseTransformVectorNoScale(FVector(TangentU));
		Output.TangentV[CopyToIdx] = (FVector3f)FromCollection.InverseTransformVectorNoScale(FVector(TangentV));
		Output.Color[CopyToIdx] = FLinearColor(AugmentedDynamicMesh::GetVertexColor(Mesh, VID));

		// Bone map is set based on the transform of the new geometry
		Output.BoneMap[CopyToIdx] = TransformIdx;
	}

	FIntVector VertexStartOffset(VerticesStart);
	for (int32 TID = 0; TID < Mesh.MaxTriangleID(); TID++)
	{
		checkSlow(Mesh.IsTriangle(TID));
		int32 CopyToIdx = FacesStart + TID;
		Output.Visible[CopyToIdx] = AugmentedDynamicMesh::GetVisibility(Mesh, TID);
		int MaterialID = Mesh.Attributes()->GetMaterialID()->GetValue(TID);
		// negative material IDs are, by convention, indications of new (internal) geometry, positive are copied through
		Output.Internal[CopyToIdx] = MaterialID < 0 ? true : AugmentedDynamicMesh::GetInternal(Mesh, TID);
		Output.MaterialID[CopyToIdx] = MaterialID < 0 ? InternalMaterialID : MaterialID;
		Output.Indices[CopyToIdx] = FIntVector(Mesh.GetTriangle(TID)) + VertexStartOffset;
	}

	if (Output.BoundingBox.Num())
	{
		Output.BoundingBox[GeometryIdx].Init();
		for (int32 Idx = VerticesStart; Idx < VerticesStart + Output.VertexCount[GeometryIdx]; ++Idx)
		{
			Output.BoundingBox[GeometryIdx] += (FVector)Output.Vertex[Idx];
		}
	}

	return true;
}


int32 FDynamicMeshCollection::AppendToCollection(const FTransform& FromCollection, FDynamicMesh3& Mesh, double CollisionSampleSpacing, int32 TransformParent, FString BoneName, FGeometryCollection& Output, int32 InternalMaterialID)
{
	if (Mesh.TriangleCount() == 0)
	{
		return -1;
	}

	if (!Mesh.IsCompact())
	{
		Mesh.CompactInPlace(nullptr);
	}

	if (CollisionSampleSpacing > 0)
	{
		AugmentedDynamicMesh::AddCollisionSamplesPerComponent(Mesh, CollisionSampleSpacing);
	}

	int32 NewGeometryStartIdx = Output.FaceStart.Num();
	int32 OriginalVertexNum = Output.Vertex.Num();
	int32 OriginalFaceNum = Output.Indices.Num();
	int32 UVLayerCount = AugmentedDynamicMesh::NumEnabledUVChannels(Mesh);
	if (!ensure(UVLayerCount > 0))
	{
		UVLayerCount = 1;
	}

	int32 GeometryIdx = Output.AddElements(1, FGeometryCollection::GeometryGroup);
	int32 TransformIdx = Output.AddElements(1, FGeometryCollection::TransformGroup);

	int32 NumTriangles = Mesh.TriangleCount();
	int32 NumVertices = Mesh.VertexCount();
	check(NumTriangles > 0);
	check(Mesh.IsCompact());
	Output.FaceCount[GeometryIdx] = NumTriangles;
	Output.FaceStart[GeometryIdx] = OriginalFaceNum;
	Output.VertexCount[GeometryIdx] = NumVertices;
	Output.VertexStart[GeometryIdx] = OriginalVertexNum;
	Output.TransformIndex[GeometryIdx] = TransformIdx;
	Output.TransformToGeometryIndex[TransformIdx] = GeometryIdx;
	if (TransformParent > -1)
	{
		Output.BoneName[TransformIdx] = BoneName;
		Output.BoneColor[TransformIdx] = Output.BoneColor[TransformParent];
		Output.Parent[TransformIdx] = TransformParent;
		Output.Children[TransformParent].Add(TransformIdx);
		Output.SimulationType[TransformParent] = FGeometryCollection::ESimulationTypes::FST_Clustered;
	}
	Output.Transform[TransformIdx] = FTransform3f::Identity;
	Output.SimulationType[TransformIdx] = FGeometryCollection::ESimulationTypes::FST_Rigid;

	int32 FacesStart = Output.AddElements(NumTriangles, FGeometryCollection::FacesGroup);
	int32 VerticesStart = Output.AddElements(NumVertices, FGeometryCollection::VerticesGroup);

	Output.SetNumUVLayers(UVLayerCount);
	GeometryCollection::UV::FUVLayers OutputUVLayers = GeometryCollection::UV::FindActiveUVLayers(Output);

	for (int32 VID = 0; VID < Mesh.MaxVertexID(); VID++)
	{
		checkSlow(Mesh.IsVertex(VID)); // mesh is compact
		int32 CopyToIdx = VerticesStart + VID;
		Output.Vertex[CopyToIdx] = (FVector3f)FromCollection.InverseTransformPosition(FVector(Mesh.GetVertex(VID)));
		Output.Normal[CopyToIdx] = (FVector3f)FromCollection.InverseTransformVectorNoScale(FVector(Mesh.GetVertexNormal(VID)));
		
		for (int32 UVLayer = 0; UVLayer < UVLayerCount; ++UVLayer)
		{
			FVector2f UV;
			AugmentedDynamicMesh::GetUV(Mesh, VID, UV, UVLayer);
			OutputUVLayers[UVLayer][CopyToIdx] = UV;
		}

		FVector3f TangentU, TangentV;
		AugmentedDynamicMesh::GetTangent(Mesh, VID, TangentU, TangentV);
		Output.TangentU[CopyToIdx] = (FVector3f)FromCollection.InverseTransformVectorNoScale(FVector(TangentU));
		Output.TangentV[CopyToIdx] = (FVector3f)FromCollection.InverseTransformVectorNoScale(FVector(TangentV));
		Output.Color[CopyToIdx] = FLinearColor(AugmentedDynamicMesh::GetVertexColor(Mesh, VID));

		// Bone map is set based on the transform of the new geometry
		Output.BoneMap[CopyToIdx] = TransformIdx;
	}

	FIntVector VertexStartOffset(VerticesStart);
	for (int32 TID = 0; TID < Mesh.MaxTriangleID(); TID++)
	{
		checkSlow(Mesh.IsTriangle(TID));
		int32 CopyToIdx = FacesStart + TID;
		Output.Visible[CopyToIdx] = AugmentedDynamicMesh::GetVisibility(Mesh, TID);
		int MaterialID = Mesh.Attributes()->GetMaterialID()->GetValue(TID);
		// negative material IDs are, by convention, indications of new (internal) geometry, positive are copied through
		Output.Internal[CopyToIdx] = MaterialID < 0 ? true : AugmentedDynamicMesh::GetInternal(Mesh, TID);
		Output.MaterialID[CopyToIdx] = MaterialID < 0 ? InternalMaterialID : MaterialID;
		Output.Indices[CopyToIdx] = FIntVector(Mesh.GetTriangle(TID)) + VertexStartOffset;
	}

	if (Output.BoundingBox.Num())
	{
		Output.BoundingBox[GeometryIdx].Init();
		for (int32 Idx = OriginalVertexNum; Idx < Output.Vertex.Num(); ++Idx)
		{
			Output.BoundingBox[GeometryIdx] += (FVector)Output.Vertex[Idx];
		}
	}

	return GeometryIdx;
}


}} // namespace UE::PlanarCut
