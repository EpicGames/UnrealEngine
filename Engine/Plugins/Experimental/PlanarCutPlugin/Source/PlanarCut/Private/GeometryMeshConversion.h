// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "DynamicMesh/DynamicMesh3.h"
#include "GeometryCollection/GeometryCollection.h"
#include "PlanarCut.h"
#include "Spatial/PointHashGrid3.h"

namespace UE::Geometry { enum class EEdgeRefineFlags; }

struct FPlanarCells;
struct FInternalSurfaceMaterials;
struct FNoiseSettings;

class FProgressCancel;

namespace UE
{
namespace PlanarCut
{

/**
 * Add attributes necessary for a dynamic mesh to represent geometry from an FGeometryCollection
 */
void SetGeometryCollectionAttributes(UE::Geometry::FDynamicMesh3& Mesh, int32 NumUVLayers);

/**
 * Clear custom FGeometryCollection-specific attributes from a FDynamicMesh3
 * Note: Does not remove the general attribute layer and MaterialID attributes, as these are not specific to geometry collections
 */
void ClearCustomGeometryCollectionAttributes(UE::Geometry::FDynamicMesh3& Mesh);

// functions for DynamicMesh3 meshes that have FGeometryCollection attributes set
namespace AugmentedDynamicMesh
{
	void SetVisibility(UE::Geometry::FDynamicMesh3& Mesh, int TID, bool bIsVisible);
	bool GetVisibility(const UE::Geometry::FDynamicMesh3& Mesh, int TID);
	void SetInternal(UE::Geometry::FDynamicMesh3& Mesh, int TID, bool bIsInternal);
	bool GetInternal(const UE::Geometry::FDynamicMesh3& Mesh, int TID);
	void SetUV(UE::Geometry::FDynamicMesh3& Mesh, int VID, FVector2f UV, int UVLayer);
	void GetUV(const UE::Geometry::FDynamicMesh3& Mesh, int VID, FVector2f& UV, int UVLayer);
	void SetTangent(UE::Geometry::FDynamicMesh3& Mesh, int VID, FVector3f Normal, FVector3f TangentU, FVector3f TangentV);
	void GetTangent(const UE::Geometry::FDynamicMesh3& Mesh, int VID, FVector3f& U, FVector3f& V);
	/// Initialize UV overlays based on the custom AugmentedDynamicMesh per-vertex UV attributes.  Optionally use FirstUVLayer to skip layers
	void InitializeOverlayToPerVertexUVs(UE::Geometry::FDynamicMesh3& Mesh, int32 NumUVLayers, int32 FirstUVLayer = 0);
	void InitializeOverlayToPerVertexTangents(UE::Geometry::FDynamicMesh3& Mesh);
	void InitializeOverlayToPerVertexColors(UE::Geometry::FDynamicMesh3& Mesh);
	void ComputeTangents(UE::Geometry::FDynamicMesh3& Mesh, bool bOnlyInternalSurfaces,
		bool bRecomputeNormals = true, bool bMakeSharpEdges = false, float SharpAngleDegrees = 60);
	void AddCollisionSamplesPerComponent(UE::Geometry::FDynamicMesh3& Mesh, double Spacing);
	void SplitOverlayAttributesToPerVertex(UE::Geometry::FDynamicMesh3& Mesh, bool bSplitUVs = true, bool bSplitTangents = true);
}


/**
 * Dynamic mesh representation of cutting cells, to be used to fracture a mesh
 */
struct FCellMeshes
{
	struct FCellInfo
	{
		UE::Geometry::FDynamicMesh3 AugMesh;

		FCellInfo(int32 NumUVLayers = 1)
		{
			SetGeometryCollectionAttributes(AugMesh, NumUVLayers);
		}

		// TODO: compute spatial in advance?  (only useful if we rework mesh booleans to support it)
		//FDynamicMeshAABBTree3 Spatial;
	};

	TIndirectArray<FCellInfo> CellMeshes;
	int32 OutsideCellIndex = -1;
	int32 NumUVLayers = 1;

	// Noise Offsets, to randomize where perlin noise is sampled
	FNoiseOffsets NoiseOffsets;

	void SetNumCells(int32 NumMeshes)
	{
		CellMeshes.Reset();
		for (int32 Idx = 0; Idx < NumMeshes; Idx++)
		{
			CellMeshes.Add(new FCellInfo(NumUVLayers));
		}
	}

	FCellMeshes(int32 NumUVLayers, FRandomStream& RandomStream)
	{
		InitEmpty(RandomStream);
	}

	FCellMeshes(int32 NumUVLayers, FRandomStream& RandomStream, const FPlanarCells& Cells, UE::Geometry::FAxisAlignedBox3d DomainBounds, double Grout, double ExtendDomain, bool bIncludeOutsideCell);

	// Note: RandomStream not required for this constructor because noise is not supported in this case
	FCellMeshes(int32 NumUVLayers, const UE::Geometry::FDynamicMesh3& SingleCutter, const FInternalSurfaceMaterials& Materials, TOptional<FTransform> Transform);

	// Special function to just make the "grout" part of the planar mesh cells
	// Used to make the multi-plane cuts with grout easier to implement
	void MakeOnlyPlanarGroutCell(const FPlanarCells& Cells, UE::Geometry::FAxisAlignedBox3d DomainBounds, double Grout);

	void RemeshForNoise(UE::Geometry::FDynamicMesh3& Mesh, UE::Geometry::EEdgeRefineFlags EdgeFlags, double TargetEdgeLen);

	FVector3d NoiseDisplacement(const FVector3d& Pos, const FNoiseSettings& Settings);

	void ApplyNoise(UE::Geometry::FDynamicMesh3& Mesh, FVector3d Normal, const FNoiseSettings& Settings, bool bProjectBoundariesToNormal = false);

	/**
	 * Convert plane index to material ID
	 * @return material ID encoding the source plane into a triangle mesh
	 */
	int PlaneToMaterial(int Plane)
	{
		return -(Plane + 1);
	}

	/**
	 * Convert material ID to plane index
	 * @return index of source plane for triangle, or -1 if no such plane
	 */
	int MaterialToPlane(int MaterialID)
	{
		return MaterialID >= 0 ? -1 : -(MaterialID + 1);
	}

	void InitEmpty(FRandomStream& RandomStream)
	{
		NoiseOffsets.SetOffsets(RandomStream);
		OutsideCellIndex = -1;
	}

	void Init(int32 NumUVLayersIn, FRandomStream& RandomStream, const FPlanarCells& Cells, UE::Geometry::FAxisAlignedBox3d DomainBounds, double Grout, double ExtendDomain, bool bIncludeOutsideCell);

	void ApplyGeneralGrout(double Grout);

	void AppendMesh(UE::Geometry::FDynamicMesh3& Base, UE::Geometry::FDynamicMesh3& ToAppend, bool bFlipped);

private:

	void CreateMeshesForBoundedPlanesWithoutNoise(int NumCells, const FPlanarCells& Cells, const UE::Geometry::FAxisAlignedBox3d& DomainBounds, bool bNoise, double GlobalUVScale);

	// Approximately calculate a "safe" spacing that would not require the remesher to create more than a million new vertices
	double GetSafeNoiseSpacing(float SurfaceArea, float TargetSpacing);

	void CreateMeshesForBoundedPlanesWithNoise(int NumCells, const FPlanarCells& Cells, const UE::Geometry::FAxisAlignedBox3d& DomainBounds, bool bNoise, double GlobalUVScale);
	void CreateMeshesForSinglePlane(const FPlanarCells& Cells, const UE::Geometry::FAxisAlignedBox3d& DomainBounds, bool bNoise, double GlobalUVScale, double Grout, bool bOnlyGrout);
};


// Holds Geometry from an FGeometryCollection in an FDynamicMesh3 representation, and convert both directions
// Also supports cutting geometry with FCellMeshes
struct FDynamicMeshCollection
{
	struct FMeshData
	{
		UE::Geometry::FDynamicMesh3 AugMesh;

		// FDynamicMeshAABBTree3 Spatial; // TODO: maybe refactor mesh booleans to allow version where caller provides spatial data; it's computed every boolean now
		// FTransform3d Transform; // TODO: maybe pretransform the data to a space that is good for cutting; refactor mesh boolean so there is an option to have it not transform input
		int32 TransformIndex; // where the mesh was from in the geometry collection
		FTransform FromCollection; // transform that was used to go from the geometry collection to the local space used for processing

		FMeshData(int32 NumUVLayers)
		{
			SetGeometryCollectionAttributes(AugMesh, NumUVLayers);
		}

		FMeshData(const UE::Geometry::FDynamicMesh3& Mesh, int32 TransformIndex, FTransform FromCollection) : AugMesh(Mesh), TransformIndex(TransformIndex), FromCollection(FromCollection)
		{}

		void SetMesh(const UE::Geometry::FDynamicMesh3& NewAugMesh)
		{
			ClearCachedBounds();
			AugMesh = NewAugMesh;
		}

		/// Note this relies on the caller to also call ClearCachedBounds() as needed; it will not automatically invalidate any computed bounds
		const UE::Geometry::FAxisAlignedBox3d& GetCachedBounds()
		{
			if (!bHasBounds)
			{
				Bounds = AugMesh.GetBounds(true);
				bHasBounds = true;
			}
			return Bounds;
		}

		void ClearCachedBounds()
		{
			bHasBounds = false;
		}

		void AddMeshToCollectionFaceMapping(int32 MeshFaceIndex, int32 CollectionFaceIndex)
		{
			MeshToCollectionFaceMapping.Add(MeshFaceIndex, CollectionFaceIndex);
		}

		int32 GetCollectionFaceFromMeshFace(int32 MeshFaceIndex) const
		{
			return MeshToCollectionFaceMapping.FindRef(MeshFaceIndex, INDEX_NONE);
		}

	private:
		bool bHasBounds = false;
		UE::Geometry::FAxisAlignedBox3d Bounds;
		// optional face mapping from the mesh to the collection ( see bGenerateMeshToCollectionFaceMapping option )
		TMap<int32, int32> MeshToCollectionFaceMapping;
	};
	TIndirectArray<FMeshData> Meshes;
	UE::Geometry::FAxisAlignedBox3d Bounds;
	
	// Settings to control the geometry import
	
	// If true, triangles where the Visible property is false will not be added to the MeshData
	bool bSkipInvisible = false;
	// If false, Transforms passed to Init are interpreted as relative to the parent bone transform. If true, Transforms are all in the same 'global' / component-relative space
	bool bComponentSpaceTransforms = false;
	bool bGenerateMeshToCollectionFaceMapping = false;
	
	FDynamicMeshCollection() {}

	FDynamicMeshCollection(const FGeometryCollection* Collection, const TArrayView<const int32>& TransformIndices, FTransform TransformCollection, bool bSaveIsolatedVertices = false)
	{
		Init(Collection, TransformIndices, TransformCollection, bSaveIsolatedVertices);
	}

	FDynamicMeshCollection(const FGeometryCollection* Collection, const TManagedArray<FTransform>& Transforms, const TArrayView<const int32>& TransformIndices, FTransform TransformCollection, bool bSaveIsolatedVertices = false)
	{
		Init(Collection, Transforms, TransformIndices, TransformCollection, bSaveIsolatedVertices);
	}

	FDynamicMeshCollection(const FGeometryCollection* Collection, const TManagedArray<FTransform3f>& Transforms, const TArrayView<const int32>& TransformIndices, FTransform TransformCollection, bool bSaveIsolatedVertices = false)
	{
		Init(Collection, Transforms, TransformIndices, TransformCollection, bSaveIsolatedVertices);
	}

	void Init(const FGeometryCollection* Collection, const TArrayView<const int32>& TransformIndices, FTransform TransformCollection, bool bSaveIsolatedVertices = false)
	{
		Init(Collection, Collection->Transform, TransformIndices, TransformCollection, bSaveIsolatedVertices);
	}

	void Init(const FGeometryCollection* Collection, const TManagedArray<FTransform>& Transforms, const TArrayView<const int32>& TransformIndices, FTransform TransformCollection, bool bSaveIsolatedVertices = false)
	{
		Init(Collection, TArrayView<const FTransform>(Transforms.GetConstArray()), TransformIndices, TransformCollection, bSaveIsolatedVertices);
	}
	void Init(const FGeometryCollection* Collection, const TManagedArray<FTransform3f>& Transforms, const TArrayView<const int32>& TransformIndices, FTransform TransformCollection, bool bSaveIsolatedVertices = false)
	{
		Init(Collection, TArrayView<const FTransform3f>(Transforms.GetConstArray()), TransformIndices, TransformCollection, bSaveIsolatedVertices);
	}

	void Init(const FGeometryCollection* Collection, TArrayView<const FTransform> Transforms, const TArrayView<const int32>& TransformIndices, FTransform TransformCollection, bool bSaveIsolatedVertices = false);
	void Init(const FGeometryCollection* Collection, TArrayView<const FTransform3f> Transforms, const TArrayView<const int32>& TransformIndices, FTransform TransformCollection, bool bSaveIsolatedVertices = false);

	int32 CutWithMultiplePlanes(
		const TArrayView<const FPlane>& Planes,
		double Grout,
		double CollisionSampleSpacing,
		const FIslandSplitSettings& IslandSplitSettings,
		int32 RandomSeed,
		FGeometryCollection* Collection,
		FInternalSurfaceMaterials& InternalSurfaceMaterials,
		bool bSetDefaultInternalMaterialsFromCollection,
		FProgressCancel* Progress = nullptr
	);

	/**
	 * Cut collection meshes with cell meshes, and append results to a geometry collection
	 *
	 * @param InternalSurfaceMaterials Internal material info (used for material ID)
	 * @param CellConnectivity The connectivity between cells: PlaneTag -> The two cells separated by triangles with this tag
	 * @param CellsMeshes Meshed versions of the cells, with noise and material properties baked in
	 * @param Collection Results will be stored in this
	 * @param bSetDefaultInternalMaterialsFromCollection If true, set internal materials to the most common external material + 1, following a convenient artist convention
	 * @param CollisionSampleSpacing If > 0, new geometry will have collision samples added (vertices not on any triangles) to fill any gaps greater than the this size
	 * @return Index of the first created geometry
	 */
	int32 CutWithCellMeshes(const FInternalSurfaceMaterials& InternalSurfaceMaterials, const TArray<TPair<int32, int32>>& CellConnectivity, FCellMeshes& CellMeshes, const FIslandSplitSettings& IslandSplitSettings, FGeometryCollection* Collection, bool bSetDefaultInternalMaterialsFromCollection, double CollisionSampleSpacing);


	/**
	 * Split islands for all collection meshes, and append results to a geometry collection
	 *
	 * @param Collection Results will be stored in this
	 * @param CollisionSampleSpacing If > 0, new geometry will have collision samples added (vertices not on any triangles) to fill any gaps greater than the this size
	 * @return Index of the first created geometry, or -1 if nothing was split
	 */
	int32 SplitAllIslands(FGeometryCollection* Collection, double CollisionSampleSpacing, const FIslandSplitSettings& IslandSplitSettings = FIslandSplitSettings());

	static void SetVisibility(FGeometryCollection& Collection, int32 GeometryIdx, bool bVisible)
	{
		int32 FaceEnd = Collection.FaceCount[GeometryIdx] + Collection.FaceStart[GeometryIdx];
		for (int32 FaceIdx = Collection.FaceStart[GeometryIdx]; FaceIdx < FaceEnd; FaceIdx++)
		{
			Collection.Visible[FaceIdx] = bVisible;
		}
	}

	// Split mesh into connected components, including implicit connections by co-located vertices
	bool SplitIslands(UE::Geometry::FDynamicMesh3& Source, TArray<TUniquePtr<UE::Geometry::FDynamicMesh3>>& SeparatedMeshes, const FIslandSplitSettings& IslandSplitSettings = FIslandSplitSettings());

	FString GetBoneName(FGeometryCollection& Output, int TransformParent, int SubPartIndex)
	{
		return Output.BoneName[TransformParent] + "_" + FString::FromInt(SubPartIndex);
	}

	void AddCollisionSamples(double CollisionSampleSpacing);

	// Update all geometry in a GeometryCollection w/ the meshes in the MeshCollection
	// Resizes the GeometryCollection as needed
	bool UpdateAllCollections(FGeometryCollection& Collection);

	static int32 AppendToCollection(const FTransform& FromCollection, UE::Geometry::FDynamicMesh3& Mesh, double CollisionSampleSpacing, int32 TransformParent, FString BoneName, FGeometryCollection& Output, int32 InternalMaterialID);

private:

	template<typename TransformType>
	void InitTemplate(const FGeometryCollection* Collection, TArrayView<const TransformType> Transforms, const TArrayView<const int32>& TransformIndices, FTransform TransformCollection, bool bSaveIsolatedVertices);


	void SetGeometryVisibility(FGeometryCollection* Collection, const TArray<int32>& GeometryIndices, bool bVisible);

	// Update an existing geometry in a collection w/ a new mesh (w/ the same number of faces and vertices!)
	static bool UpdateCollection(const FTransform& FromCollection, UE::Geometry::FDynamicMesh3& Mesh, int32 GeometryIdx, FGeometryCollection& Output, int32 InternalMaterialID);

	void FillVertexHash(const UE::Geometry::FDynamicMesh3& Mesh, UE::Geometry::TPointHashGrid3d<int>& VertHash);

	bool IsNeighboring(
		UE::Geometry::FDynamicMesh3& MeshA, const UE::Geometry::TPointHashGrid3d<int>& VertHashA, const UE::Geometry::FAxisAlignedBox3d& BoundsA,
		UE::Geometry::FDynamicMesh3& MeshB, const UE::Geometry::TPointHashGrid3d<int>& VertHashB, const UE::Geometry::FAxisAlignedBox3d& BoundsB);
};


}} // namespace UE::PlanarCut
