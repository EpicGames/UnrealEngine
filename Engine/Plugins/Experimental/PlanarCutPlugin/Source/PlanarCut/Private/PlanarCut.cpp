// Copyright Epic Games, Inc. All Rights Reserved.
#include "PlanarCut.h"
#include "GeometryCollection/Facades/CollectionMeshFacade.h"
#include "PlanarCutPlugin.h"

#include "Spatial/SparseDynamicOctree3.h"
#include "Spatial/MeshAABBTree3.h"
#include "MeshQueries.h"
#include "Util/IndexUtil.h"

#include "GeometryCollection/GeometryCollectionAlgo.h"
#include "GeometryCollection/GeometryCollectionClusteringUtility.h"
#include "GeometryCollection/GeometryCollectionConvexUtility.h"


#include "DynamicMeshEditor.h"
#include "DynamicMesh/MeshTransforms.h"
#include "Operations/MeshBoolean.h"
#include "DynamicMesh/Operations/MergeCoincidentMeshEdges.h"
#include "DynamicMesh/MeshNormals.h"
#include "DynamicMesh/MeshTangents.h"

#include "StaticMeshOperations.h"
#include "MeshDescriptionToDynamicMesh.h"
#include "DynamicMeshToMeshDescription.h"

#include "Algo/Reverse.h"

#include "GeometryMeshConversion.h"
#include "Voronoi/Voronoi.h"

using namespace UE::Geometry;


using namespace UE::PlanarCut;

#define LOCTEXT_NAMESPACE "PlanarCut"

namespace PlanarCut_Locals
{
	FVector SeparateTranslation(const FTransform& Transform, FTransform& OutCenteredTransform)
	{
		FVector Translation = Transform.GetTranslation();
		OutCenteredTransform = FTransform(Transform.GetRotation(), FVector::ZeroVector, Transform.GetScale3D());
		return Translation;
	}
}

// logic from FMeshUtility::GenerateGeometryCollectionFromBlastChunk, sets material IDs based on construction pattern that external materials have even IDs and are matched to internal materials at InternalID = ExternalID+1
int32 FInternalSurfaceMaterials::GetDefaultMaterialIDForGeometry(const FGeometryCollection& Collection, int32 GeometryIdx) const
{
	auto FindMostCommonInternalMaterial = [](const FGeometryCollection& InCollection, int32 InFaceStart, int32 InFaceEnd) -> int32
	{
		// find most common internal material
		TMap<int32, int32> MaterialIDCount;
		int32 MaxCount = 0;
		int32 MostCommonMaterialID = INDEX_NONE;
		const TManagedArray<int32>& MaterialID = InCollection.MaterialID;
		const TManagedArray<bool>& Internal = InCollection.Internal;
		for (int i = InFaceStart; i < InFaceEnd; ++i)
		{
			if (!Internal[i])
			{
				continue;
			}
			int32 CurrID = MaterialID[i];
			int32& CurrCount = MaterialIDCount.FindOrAdd(CurrID);
			CurrCount++;

			if (CurrCount > MaxCount)
			{
				MaxCount = CurrCount;
				MostCommonMaterialID = CurrID;
			}
		}
		return MostCommonMaterialID;
	};

	int32 MostCommonMaterialID = INDEX_NONE;
	int32 AllFaceStart = 0;
	int32 AllFaceEnd = Collection.Indices.Num();
	if (GeometryIdx > -1)
	{
		int32 GeoFaceStart = Collection.FaceStart[GeometryIdx];
		int32 GeoFaceEnd = Collection.FaceCount[GeometryIdx] + Collection.FaceStart[GeometryIdx];
		MostCommonMaterialID = FindMostCommonInternalMaterial(Collection, GeoFaceStart, GeoFaceEnd);
	}
	if (MostCommonMaterialID == INDEX_NONE)
	{
		MostCommonMaterialID = FindMostCommonInternalMaterial(Collection, AllFaceStart, AllFaceEnd);

		// Failed to find any internal material IDs; default to the first material
		if (MostCommonMaterialID == INDEX_NONE)
		{
			MostCommonMaterialID = 0;
		}
	}

	return MostCommonMaterialID;
}

void FInternalSurfaceMaterials::SetUVScaleFromCollection(const GeometryCollection::Facades::FCollectionMeshFacade& CollectionMesh, int32 GeometryIdx)
{
	const auto& VertexArray = CollectionMesh.VertexAttribute.Get();
	const auto& UV0Array = CollectionMesh.GetUVLayer(0);
	const auto& IndicesArray = CollectionMesh.IndicesAttribute.Get();
	const auto& FaceStartArray = CollectionMesh.FaceStartAttribute.Get();
	const auto& FaceCountArray = CollectionMesh.FaceCountAttribute.Get();
	
	int32 FaceStart = 0;
	int32 FaceEnd = IndicesArray.Num();
	if (GeometryIdx > -1)
	{
		FaceStart = FaceStartArray[GeometryIdx];
		FaceEnd = FaceCountArray[GeometryIdx] + FaceStartArray[GeometryIdx];
	}
	double UVDistance = 0;
	float WorldDistance = 0;
	for (int32 FaceIdx = FaceStart; FaceIdx < FaceEnd; FaceIdx++)
	{
		const FIntVector& Tri = IndicesArray[FaceIdx];
		WorldDistance += FVector3f::Distance(VertexArray[Tri.X], VertexArray[Tri.Y]);
		UVDistance += FVector2D::Distance(FVector2D(UV0Array[Tri.X]), FVector2D(UV0Array[Tri.Y]));
		WorldDistance += FVector3f::Distance(VertexArray[Tri.Z], VertexArray[Tri.Y]);
		UVDistance += FVector2D::Distance(FVector2D(UV0Array[Tri.Z]), FVector2D(UV0Array[Tri.Y]));
		WorldDistance += FVector3f::Distance(VertexArray[Tri.X], VertexArray[Tri.Z]);
		UVDistance += FVector2D::Distance(FVector2D(UV0Array[Tri.X]), FVector2D(UV0Array[Tri.Z]));
	}

	ensure(FMath::IsFinite(GlobalUVScale));
	if (WorldDistance > 0)
	{
		GlobalUVScale =  static_cast<float>(UVDistance) / WorldDistance;
	}
	ensure(FMath::IsFinite(GlobalUVScale));
	if (GlobalUVScale <= 0)
	{
		GlobalUVScale = 1;
	}
	ensure(FMath::IsFinite(GlobalUVScale));
}



FPlanarCells::FPlanarCells(const FPlane& P)
{
	NumCells = 2;
	AddPlane(P, 0, 1);
}

FPlanarCells::FPlanarCells(const TArrayView<const FVector> Sites, FVoronoiDiagram& Voronoi)
{
	TArray<FVoronoiCellInfo> VoronoiCells;
	Voronoi.ComputeAllCells(VoronoiCells);

	AssumeConvexCells = true;
	NumCells = VoronoiCells.Num();
	for (int32 CellIdx = 0; CellIdx < NumCells; CellIdx++)
	{
		int32 LocalVertexStart = -1;

		const FVoronoiCellInfo& CellInfo = VoronoiCells[CellIdx];
		int32 CellFaceVertexIndexStart = 0;
		for (int32 CellFaceIdx = 0; CellFaceIdx < CellInfo.Neighbors.Num(); CellFaceIdx++, CellFaceVertexIndexStart += 1 + CellInfo.Faces[CellFaceVertexIndexStart])
		{
			int32 NeighborIdx = CellInfo.Neighbors[CellFaceIdx];
			if (CellIdx < NeighborIdx)  // Filter out faces that we expect to get by symmetry
			{
				continue;
			}

			FVector Normal = CellInfo.Normals[CellFaceIdx];
			if (Normal.IsZero())
			{
				if (NeighborIdx > -1)
				{
					Normal = Sites[NeighborIdx] - Sites[CellIdx];
					bool bNormalizeSucceeded = Normal.Normalize();
					ensureMsgf(bNormalizeSucceeded, TEXT("Voronoi diagram should not have Voronoi sites so close together!"));
				}
				else
				{
					// degenerate face on border; likely almost zero area so hopefully it won't matter if we just don't add it
					continue;
				}
			}
			FPlane P(Normal, FVector::DotProduct(Normal, CellInfo.Vertices[CellInfo.Faces[CellFaceVertexIndexStart + 1]]));
			if (LocalVertexStart < 0)
			{
				LocalVertexStart = PlaneBoundaryVertices.Num();
				PlaneBoundaryVertices.Append(CellInfo.Vertices);
			}
			TArray<int32> PlaneBoundary;
			int32 FaceSize = CellInfo.Faces[CellFaceVertexIndexStart];
			for (int32 i = 0; i < FaceSize; i++)
			{
				int32 CellVertexIdx = CellInfo.Faces[CellFaceVertexIndexStart + 1 + i];
				PlaneBoundary.Add(LocalVertexStart + CellVertexIdx);
			}

			AddPlane(P, CellIdx, NeighborIdx, PlaneBoundary);
		}
	}
}

FPlanarCells::FPlanarCells(const TArrayView<const FBox> Boxes, bool bResolveAdjacencies)
{
	AssumeConvexCells = true;
	NumCells = Boxes.Num();
	TArray<FBox> BoxesCopy(Boxes);
	
	if (!bResolveAdjacencies) // if the boxes aren't touching, we can just make a completely independent cell per box
	{
		for (int32 BoxIdx = 0; BoxIdx < NumCells; BoxIdx++)
		{
			const FBox& Box = Boxes[BoxIdx];
			const FVector& Min = Box.Min;
			const FVector& Max = Box.Max;

			int32 VIdx = PlaneBoundaryVertices.Num();
			PlaneBoundaryVertices.Add(Min);
			PlaneBoundaryVertices.Add(FVector(Max.X, Min.Y, Min.Z));
			PlaneBoundaryVertices.Add(FVector(Max.X, Max.Y, Min.Z));
			PlaneBoundaryVertices.Add(FVector(Min.X, Max.Y, Min.Z));

			PlaneBoundaryVertices.Add(FVector(Min.X, Min.Y, Max.Z));
			PlaneBoundaryVertices.Add(FVector(Max.X, Min.Y, Max.Z));
			PlaneBoundaryVertices.Add(Max);
			PlaneBoundaryVertices.Add(FVector(Min.X, Max.Y, Max.Z));

			AddPlane(FPlane(FVector(0, 0, -1), -Min.Z), BoxIdx, -1, { VIdx + 0, VIdx + 1, VIdx + 2, VIdx + 3 });
			AddPlane(FPlane(FVector(0, 0, 1), Max.Z), BoxIdx, -1, { VIdx + 4, VIdx + 7, VIdx + 6, VIdx + 5 });
			AddPlane(FPlane(FVector(0, -1, 0), -Min.Y), BoxIdx, -1, { VIdx + 0, VIdx + 4, VIdx + 5, VIdx + 1 });
			AddPlane(FPlane(FVector(0, 1, 0), Max.Y), BoxIdx, -1, { VIdx + 3, VIdx + 2, VIdx + 6, VIdx + 7 });
			AddPlane(FPlane(FVector(-1, 0, 0), -Min.X), BoxIdx, -1, { VIdx + 0, VIdx + 3, VIdx + 7, VIdx + 4 });
			AddPlane(FPlane(FVector(1, 0, 0), Max.X), BoxIdx, -1, { VIdx + 1, VIdx + 5, VIdx + 6, VIdx + 2 });
		}

		return;
	}

	// If boxes might be touching, we need to subdivide each box along each overlap,
	// and make sure to only construct one copy any vertex/face that is shared between multiple boxes

	// Build an octree of boxes to allow us to quickly find neighbors, and determine how to subdivide the boxes
	FSparseDynamicOctree3 BoxTree;
	double BoxMaxDim = 0;
	for (const FBox& Box : Boxes)
	{
		BoxMaxDim = FMath::Max(BoxMaxDim, Box.GetSize().GetMax());
	}
	BoxTree.RootDimension = BoxMaxDim * 4;
	for (int32 BoxIdx = 0; BoxIdx < NumCells; BoxIdx++)
	{
		BoxTree.InsertObject(BoxIdx, Boxes[BoxIdx]);
	}

	// Build a hash grid for vertices, to share vertices across boxes
	constexpr double AddVertTolerance = UE_DOUBLE_KINDA_SMALL_NUMBER;
	TPointHashGrid3d<int32> PosToVert(AddVertTolerance * 10, INDEX_NONE);
	TMap<FIntVector4, int32> VertsToPlane;

	// Track overlaps between neighboring boxes in each dimension separately; each overlap is a new subdivision on that axis
	TArray<int32> OverlapIndices;
	TArray<double> Overlaps[3];
	for (int32 BoxIdx = 0; BoxIdx < NumCells; BoxIdx++)
	{
		const FBox& Box = Boxes[BoxIdx];
		const FVector& Min = Box.Min;
		const FVector& Max = Box.Max;

		Overlaps[0].Reset();
		Overlaps[1].Reset();
		Overlaps[2].Reset();
		OverlapIndices.Reset();
		
		constexpr double QueryExpandTolerance = AddVertTolerance;
		
		FBox ExpandedBox = Box.ExpandBy(QueryExpandTolerance);
		BoxTree.RangeQuery(ExpandedBox, OverlapIndices);
		for (int32 OverlapIdx : OverlapIndices)
		{
			// skip if same box or not an actual overlap
			const FBox& OtherBox = Boxes[OverlapIdx];
			if (BoxIdx == OverlapIdx || !ExpandedBox.Intersect(OtherBox))
			{
				continue;
			}

			auto AddIfInRange = [AddVertTolerance](double Val, double RangeMin, double RangeMax, TArray<double>& AddTo)
			{
				if (Val > RangeMin + AddVertTolerance && Val < RangeMax - AddVertTolerance)
				{
					AddTo.Add(Val);
				}
			};

			for (int32 Dim = 0; Dim < 3; ++Dim)
			{
				AddIfInRange(OtherBox.Min[Dim], Box.Min[Dim], Box.Max[Dim], Overlaps[Dim]);
				AddIfInRange(OtherBox.Max[Dim], Box.Min[Dim], Box.Max[Dim], Overlaps[Dim]);
			}
		}

		for (int32 Dim = 0; Dim < 3; ++Dim)
		{
			Overlaps[Dim].Add(Box.Min[Dim]);
			Overlaps[Dim].Add(Box.Max[Dim]);
			Overlaps[Dim].Sort();

			double Last = Overlaps[Dim][0];
			int32 FillIdx = 1;
			for (int32 Idx = 1; Idx < Overlaps[Dim].Num(); ++Idx)
			{
				if (Overlaps[Dim][Idx] - Last >= AddVertTolerance)
				{
					Overlaps[Dim][FillIdx] = Overlaps[Dim][Idx];
					Last = Overlaps[Dim][Idx];
					++FillIdx;
				}
				// else overlap was too close to 'last', so we skip it
			}
			Overlaps[Dim].SetNum(FillIdx, EAllowShrinking::No);
			if (Overlaps[Dim].Num() == 1)
			{
				Overlaps[Dim].Add(Box.Max[Dim]);
			}
			else
			{
				// make sure to always end at the exact max
				Overlaps[Dim].Last() = Box.Max[Dim];
			}
		}

		// Helper to get a shared vertex, first by looking if we've already made it locally, then by checking the global hash grid
		TMap<FIndex3i, int32> GridToVert; // A local map for all the vertices we've already found on the box
		auto GetVertex = [this, &Overlaps, &GridToVert, &PosToVert, AddVertTolerance](int32 FaceDim, int32 UDim, int32 VDim, int32 FaceCoord, int32 U, int32 V)
		{
			FIndex3i Coord;
			Coord[FaceDim] = FaceCoord;
			Coord[UDim] = U;
			Coord[VDim] = V;
			int32* FoundV = GridToVert.Find(Coord);
			if (FoundV)
			{
				return *FoundV;
			}
			FVector Pos(Overlaps[0][Coord.A], Overlaps[1][Coord.B], Overlaps[2][Coord.C]);
			TPair<int32, double> Found = PosToVert.FindNearestInRadius(Pos, AddVertTolerance, [&](const int32& Idx) {return FVector::DistSquared(Pos, PlaneBoundaryVertices[Idx]);});
			if (Found.Key != INDEX_NONE)
			{
				return Found.Key;
			}

			int32 NewV = PlaneBoundaryVertices.Add(Pos);
			GridToVert.Add(Coord, NewV);
			PosToVert.InsertPointUnsafe(NewV, Pos);
			return NewV;
		};

		// Now that we have decided how to subdivide each dimension (w/ the Overlaps arrays), we need to construct each face (or link to the face, if it already exists)
		for (int32 FaceDim = 0; FaceDim < 3; ++FaceDim)
		{
			for (int32 Side = 0; Side < 2; ++Side)
			{
				int32 UDim = (FaceDim + 1) % 3;
				int32 VDim = (FaceDim + 2) % 3;
				int32 FaceCoord = (Side == 1) ? (Overlaps[FaceDim].Num() - 1) : 0;
				double FaceSign = double(Side * 2 - 1);
				FVector Normal(0, 0, 0);
				Normal[FaceDim] = FaceSign;
				double PlaneW = FaceSign * (Side == 0 ? Min[FaceDim] : Max[FaceDim]);
				for (int32 U = 0; U + 1 < Overlaps[UDim].Num(); ++U)
				{
					for (int32 V = 0; V + 1 < Overlaps[VDim].Num(); ++V)
					{
						int32 V00 = GetVertex(FaceDim, UDim, VDim, FaceCoord, U, V);
						int32 V10 = GetVertex(FaceDim, UDim, VDim, FaceCoord, U + 1, V);
						int32 V11 = GetVertex(FaceDim, UDim, VDim, FaceCoord, U + 1, V + 1);
						int32 V01 = GetVertex(FaceDim, UDim, VDim, FaceCoord, U, V + 1);

						FIntVector4 FaceKey(V00, V10, V11, V01);
						int32* FoundPlane = VertsToPlane.Find(FaceKey);
						if (FoundPlane)
						{
							PlaneCells[*FoundPlane].Value = BoxIdx;
							continue;
						}

						TArray<int32> BoundaryVerts;
						if (Side == 0)
						{
							BoundaryVerts = { V00, V10, V11, V01 };
						}
						else // reverse winding for the 'far' face
						{
							BoundaryVerts = { V10, V00, V01, V11 };
						}
						VertsToPlane.Add(FaceKey, AddPlane(FPlane(Normal, PlaneW), BoxIdx, -1, BoundaryVerts));
					}
				}
			}
		}
	}

}

FPlanarCells::FPlanarCells(const FBox& Region, const FIntVector& CubesPerAxis)
{
	AssumeConvexCells = true;
	NumCells = CubesPerAxis.X * CubesPerAxis.Y * CubesPerAxis.Z;

	// cube X, Y, Z integer indices to a single cell index
	auto ToIdx = [](const FIntVector &PerAxis, int32 Xi, int32 Yi, int32 Zi)
	{
		if (Xi < 0 || Xi >= PerAxis.X || Yi < 0 || Yi >= PerAxis.Y || Zi < 0 || Zi >= PerAxis.Z)
		{
			return -1;
		}
		else
		{
			return Xi + Yi * (PerAxis.X) + Zi * (PerAxis.X * PerAxis.Y);
		}
	};

	auto ToIdxUnsafe = [](const FIntVector &PerAxis, int32 Xi, int32 Yi, int32 Zi)
	{
		return Xi + Yi * (PerAxis.X) + Zi * (PerAxis.X * PerAxis.Y);
	};

	FIntVector VertsPerAxis = CubesPerAxis + FIntVector(1);
	PlaneBoundaryVertices.SetNum(VertsPerAxis.X * VertsPerAxis.Y * VertsPerAxis.Z);

	FVector Diagonal = Region.Max - Region.Min;
	FVector CellSizes(
		Diagonal.X / CubesPerAxis.X,
		Diagonal.Y / CubesPerAxis.Y,
		Diagonal.Z / CubesPerAxis.Z
	);
	int32 VertIdx = 0;
	for (int32 Zi = 0; Zi < VertsPerAxis.Z; Zi++)
	{
		for (int32 Yi = 0; Yi < VertsPerAxis.Y; Yi++)
		{
			for (int32 Xi = 0; Xi < VertsPerAxis.X; Xi++)
			{
				PlaneBoundaryVertices[VertIdx] = Region.Min + FVector(Xi * CellSizes.X, Yi * CellSizes.Y, Zi * CellSizes.Z);
				ensure(VertIdx == ToIdxUnsafe(VertsPerAxis, Xi, Yi, Zi));
				VertIdx++;
			}
		}
	}
	float Z = static_cast<float>( Region.Min.Z );
	int32 ZSliceSize = VertsPerAxis.X * VertsPerAxis.Y;
	int32 VIdxOffs[8] = { 0, 1, VertsPerAxis.X + 1, VertsPerAxis.X, ZSliceSize, ZSliceSize + 1, ZSliceSize + VertsPerAxis.X + 1, ZSliceSize + VertsPerAxis.X };
	for (int32 Zi = 0; Zi < CubesPerAxis.Z; Zi++, Z += static_cast<float>(CellSizes.Z))
	{
		float Y = static_cast<float>( Region.Min.Y );
		float ZN = Z + static_cast<float>(CellSizes.Z);
		for (int32 Yi = 0; Yi < CubesPerAxis.Y; Yi++, Y += static_cast<float>(CellSizes.Y))
		{
			float X = static_cast<float>(Region.Min.X);
			float YN = Y + static_cast<float>(CellSizes.Y);
			for (int32 Xi = 0; Xi < CubesPerAxis.X; Xi++, X += static_cast<float>(CellSizes.X))
			{
				float XN = X + static_cast<float>( CellSizes.X );
				int VIdx = ToIdxUnsafe(VertsPerAxis, Xi, Yi, Zi);
				int BoxIdx = ToIdxUnsafe(CubesPerAxis, Xi, Yi, Zi);

				AddPlane(FPlane(FVector(0, 0, -1), -Z), BoxIdx, ToIdx(CubesPerAxis, Xi, Yi, Zi-1), { VIdx + VIdxOffs[0], VIdx + VIdxOffs[1], VIdx + VIdxOffs[2], VIdx + VIdxOffs[3] });
				AddPlane(FPlane(FVector(0, 0, 1), ZN), BoxIdx, ToIdx(CubesPerAxis, Xi, Yi, Zi+1), { VIdx + VIdxOffs[4], VIdx + VIdxOffs[7], VIdx + VIdxOffs[6], VIdx + VIdxOffs[5] });
				AddPlane(FPlane(FVector(0, -1, 0), -Y), BoxIdx, ToIdx(CubesPerAxis, Xi, Yi-1, Zi), { VIdx + VIdxOffs[0], VIdx + VIdxOffs[4], VIdx + VIdxOffs[5], VIdx + VIdxOffs[1] });
				AddPlane(FPlane(FVector(0, 1, 0), YN), BoxIdx, ToIdx(CubesPerAxis, Xi, Yi+1, Zi), { VIdx + VIdxOffs[3], VIdx + VIdxOffs[2], VIdx + VIdxOffs[6], VIdx + VIdxOffs[7] });
				AddPlane(FPlane(FVector(-1, 0, 0), -X), BoxIdx, ToIdx(CubesPerAxis, Xi-1, Yi, Zi), { VIdx + VIdxOffs[0], VIdx + VIdxOffs[3], VIdx + VIdxOffs[7], VIdx + VIdxOffs[4] });
				AddPlane(FPlane(FVector(1, 0, 0), XN), BoxIdx, ToIdx(CubesPerAxis, Xi+1, Yi, Zi), { VIdx + VIdxOffs[1], VIdx + VIdxOffs[5], VIdx + VIdxOffs[6], VIdx + VIdxOffs[2] });
			}
		}
	}
}

FPlanarCells::FPlanarCells(const FBox &Region, const TArrayView<const FColor> Image, int32 Width, int32 Height)
{
	const double SimplificationTolerance = 0.0; // TODO: implement simplification and make tolerance a param

	const FColor OutsideColor(0, 0, 0);

	int32 NumPix = Width * Height;
	check(Image.Num() == NumPix);

	// Union Find adapted from PBDRigidClustering.cpp version; customized to pixel grouping
	struct UnionFindInfo
	{
		int32 GroupIdx;
		int32 Size;
	};

	TArray<UnionFindInfo> PixCellUnions; // union find info per pixel
	TArray<int32> PixCells;  // Cell Index per pixel (-1 for OutsideColor pixels)

	PixCellUnions.SetNumUninitialized(NumPix);
	PixCells.SetNumUninitialized(NumPix);
	for (int32 i = 0; i < NumPix; ++i)
	{
		if (Image[i] == OutsideColor)
		{
			PixCellUnions[i].GroupIdx = -1;
			PixCellUnions[i].Size = 0;
			PixCells[i] = -1;
		}
		else
		{
			PixCellUnions[i].GroupIdx = i;
			PixCellUnions[i].Size = 1;
			PixCells[i] = -2;
		}
	}
	auto FindGroup = [&](int Idx) {
		int GroupIdx = Idx;

		int findIters = 0;
		while (PixCellUnions[GroupIdx].GroupIdx != GroupIdx)
		{
			ensure(findIters++ < 10); // if this while loop iterates more than a few times, there's probably a bug in the unionfind
			PixCellUnions[GroupIdx].GroupIdx = PixCellUnions[PixCellUnions[GroupIdx].GroupIdx].GroupIdx;
			GroupIdx = PixCellUnions[GroupIdx].GroupIdx;
		}

		return GroupIdx;
	};
	auto MergeGroup = [&](int A, int B) {
		int GroupA = FindGroup(A);
		int GroupB = FindGroup(B);
		if (GroupA == GroupB)
		{
			return;
		}
		if (PixCellUnions[GroupA].Size > PixCellUnions[GroupB].Size)
		{
			Swap(GroupA, GroupB);
		}
		PixCellUnions[GroupA].GroupIdx = GroupB;
		PixCellUnions[GroupB].Size += PixCellUnions[GroupA].Size;
	};
	// merge non-outside neighbors into groups
	int32 YOffs[4] = { -1, 0, 0, 1 };
	int32 XOffs[4] = { 0, -1, 1, 0 };
	for (int32 Yi = 0; Yi < Height; Yi++)
	{
		for (int32 Xi = 0; Xi < Width; Xi++)
		{
			int32 Pi = Xi + Yi * Width;
			if (PixCells[Pi] == -1) // outside cell
			{
				continue;
			}
			for (int Oi = 0; Oi < 4; Oi++)
			{
				int32 Yn = Yi + YOffs[Oi];
				int32 Xn = Xi + XOffs[Oi];
				int32 Pn = Xn + Yn * Width;
				if (Xn < 0 || Xn >= Width || Yn < 0 || Yn >= Height || PixCells[Pn] == -1) // outside nbr
				{
					continue;
				}
				
				MergeGroup(Pi, Pn);
			}
		}
	}
	// assign cell indices from compacted group IDs
	NumCells = 0;
	for (int32 Pi = 0; Pi < NumPix; Pi++)
	{
		if (PixCells[Pi] == -1)
		{
			continue;
		}
		int32 GroupID = FindGroup(Pi);
		if (PixCells[GroupID] == -2)
		{
			PixCells[GroupID] = NumCells++;
		}
		PixCells[Pi] = PixCells[GroupID];
	}

	// Dimensions of pixel corner data
	int32 CWidth = Width + 1;
	int32 CHeight = Height + 1;
	int32 NumCorners = CWidth * CHeight;
	TArray<int32> CornerIndices;
	CornerIndices.SetNumZeroed(NumCorners);

	TArray<TMap<int32, TArray<int32>>> PerCellBoundaryEdgeArrays;
	TArray<TArray<TArray<int32>>> CellBoundaryCorners;
	PerCellBoundaryEdgeArrays.SetNum(NumCells);
	CellBoundaryCorners.SetNum(NumCells);
	
	int32 COffX1[4] = { 1,0,1,0 };
	int32 COffX0[4] = { 0,0,1,1 };
	int32 COffY1[4] = { 0,0,1,1 };
	int32 COffY0[4] = { 0,1,0,1 };
	for (int32 Yi = 0; Yi < Height; Yi++)
	{
		for (int32 Xi = 0; Xi < Width; Xi++)
		{
			int32 Pi = Xi + Yi * Width;
			int32 Cell = PixCells[Pi];
			if (Cell == -1) // outside cell
			{
				continue;
			}
			for (int Oi = 0; Oi < 4; Oi++)
			{
				int32 Yn = Yi + YOffs[Oi];
				int32 Xn = Xi + XOffs[Oi];
				int32 Pn = Xn + Yn * Width;
				
				// boundary edge found
				if (Xn < 0 || Xn >= Width || Yn < 0 || Yn >= Height || PixCells[Pn] != PixCells[Pi])
				{
					int32 C0 = Xi + COffX0[Oi] + CWidth * (Yi + COffY0[Oi]);
					int32 C1 = Xi + COffX1[Oi] + CWidth * (Yi + COffY1[Oi]);
					TArray<int32> Chain = { C0, C1 };
					int32 Last;
					while (PerCellBoundaryEdgeArrays[Cell].Contains(Last = Chain.Last()))
					{
						Chain.Pop(EAllowShrinking::No);
						Chain.Append(PerCellBoundaryEdgeArrays[Cell][Last]);
						PerCellBoundaryEdgeArrays[Cell].Remove(Last);
					}
					if (Last == C0)
					{
						CellBoundaryCorners[Cell].Add(Chain);
					}
					else
					{
						PerCellBoundaryEdgeArrays[Cell].Add(Chain[0], Chain);
					}
				}
			}
		}
	}

	FVector RegionDiagonal = Region.Max - Region.Min;

	for (int32 CellIdx = 0; CellIdx < NumCells; CellIdx++)
	{
		ensure(CellBoundaryCorners[CellIdx].Num() > 0); // there must not be any regions with no boundary
		ensure(PerCellBoundaryEdgeArrays[CellIdx].Num() == 0); // all boundary edge array should have been consumed and turned to full boundary loops
		ensureMsgf(CellBoundaryCorners[CellIdx].Num() == 1, TEXT("Have not implemented support for regions with holes!"));

		int32 BoundaryStart = PlaneBoundaryVertices.Num();
		const TArray<int32>& Bounds = CellBoundaryCorners[CellIdx][0];
		int32 Dx = 0, Dy = 0;
		auto CornerIdxToPos = [&](int32 CornerID)
		{
			int32 Xi = CornerID % CWidth;
			int32 Yi = CornerID / CWidth;
			return FVector2D(
				Region.Min.X + Xi * RegionDiagonal.X / float(Width),
				Region.Min.Y + Yi * RegionDiagonal.Y / float(Height)
			);
		};
		
		FVector2D LastP = CornerIdxToPos(Bounds[0]);
		int32 NumBoundVerts = 0;
		TArray<int32> FrontBound;
		for (int32 BoundIdx = 1; BoundIdx < Bounds.Num(); BoundIdx++)
		{
			FVector2D NextP = CornerIdxToPos(Bounds[BoundIdx]);
			FVector2D Dir = NextP - LastP;
			Dir.Normalize();
			int BoundSkip = BoundIdx;
			while (++BoundSkip < Bounds.Num())
			{
				FVector2D SkipP = CornerIdxToPos(Bounds[BoundSkip]);
				if (FVector2D::DotProduct(SkipP - NextP, Dir) < 1e-6)
				{
					break;
				}
				NextP = SkipP;
				BoundIdx = BoundSkip;
			}
			PlaneBoundaryVertices.Add(FVector(NextP.X, NextP.Y, Region.Min.Z));
			PlaneBoundaryVertices.Add(FVector(NextP.X, NextP.Y, Region.Max.Z));
			int32 Front = BoundaryStart + NumBoundVerts * 2;
			int32 Back = Front + 1;
			FrontBound.Add(Front);
			if (NumBoundVerts > 0)
			{
				AddPlane(FPlane(PlaneBoundaryVertices.Last(), FVector(Dir.Y, -Dir.X, 0)), CellIdx, -1, {Back, Front, Front - 2, Back - 2});
			}

			NumBoundVerts++;
			LastP = NextP;
		}

		// add the last edge, connecting the start and end
		FVector2D Dir = CornerIdxToPos(Bounds[1]) - LastP;
		Dir.Normalize();
		AddPlane(FPlane(PlaneBoundaryVertices.Last(), FVector(Dir.Y, -Dir.X, 0)), CellIdx, -1, {BoundaryStart+1, BoundaryStart, BoundaryStart+NumBoundVerts*2-2, BoundaryStart+NumBoundVerts*2-1});

		// add the front and back faces
		AddPlane(FPlane(Region.Min, FVector(0, 0, -1)), CellIdx, -1, FrontBound);
		TArray<int32> BackBound; BackBound.SetNum(FrontBound.Num());
		for (int32 Idx = 0, N = BackBound.Num(); Idx < N; Idx++)
		{
			BackBound[Idx] = FrontBound[N - 1 - Idx] + 1;
		}
		AddPlane(FPlane(Region.Max, FVector(0, 0, 1)), CellIdx, -1, BackBound);
	}


	AssumeConvexCells = false; // todo could set this to true if the 2D shape of each image region is convex
}


void FPlanarCells::DiscardCells(TFunctionRef<bool(int32)> KeepFunc, bool bKeepNeighbors)
{
	TArray<int32> OldToNew;
	OldToNew.Init(-1, NumCells);
	int32 KeptCells = 0;
	for (int32 CellIdx = 0; CellIdx < NumCells; ++CellIdx)
	{
		if (KeepFunc(CellIdx))
		{
			OldToNew[CellIdx] = KeptCells++;
		}
	}
	if (bKeepNeighbors && KeptCells < NumCells)
	{
		for (const TPair<int32, int32>& Neighbors : PlaneCells)
		{
			if (Neighbors.Key < 0 || Neighbors.Value < 0)
			{
				continue;
			}
			bool bKeptKey = KeepFunc(Neighbors.Key);
			bool bKeptValue = KeepFunc(Neighbors.Value);
			if (bKeptKey != bKeptValue)
			{
				int32 Nbr = bKeptKey ? Neighbors.Value : Neighbors.Key;
				if (OldToNew[Nbr] < 0)
				{
					OldToNew[Nbr] = KeptCells++;
				}
			}
		}
	}
	if (KeptCells == NumCells)
	{
		return;
	}

	NumCells = KeptCells;
	for (int32 PlaneIdx = 0; PlaneIdx < Planes.Num(); ++PlaneIdx)
	{
		TPair<int32, int32> Cells = PlaneCells[PlaneIdx];
		Cells.Key = Cells.Key > -1 ? OldToNew[Cells.Key] : -1;
		Cells.Value = Cells.Value > -1 ? OldToNew[Cells.Value] : -1;
		if (Cells.Key == Cells.Value && Cells.Key == -1)
		{
			PlaneCells.RemoveAtSwap(PlaneIdx, EAllowShrinking::No);
			Planes.RemoveAtSwap(PlaneIdx, EAllowShrinking::No);
			PlaneBoundaries.RemoveAtSwap(PlaneIdx, EAllowShrinking::No);
			PlaneIdx--; // consider the swapped-in value in the next iteration
		}
		else
		{
			// on boundary to outside, the 'outside' index must always be second
			// if the discards above broke that invariant, flip the plane to fix it
			if (Cells.Key < 0)
			{
				Swap(Cells.Key, Cells.Value);
				Algo::Reverse(PlaneBoundaries[PlaneIdx]);
				Planes[PlaneIdx] = Planes[PlaneIdx].Flip();
			}
			PlaneCells[PlaneIdx] = Cells;
		}
	}

	// Compress vertices array to only the used vertices
	TArray<int32> OldToNewVertex;
	OldToNewVertex.Init(-1, PlaneBoundaryVertices.Num());
	TArray<FVector> NewBoundaryVertices;
	for (TArray<int32>& PlaneBoundary : PlaneBoundaries)
	{
		for (int32& VID : PlaneBoundary)
		{
			int32& NewVID = OldToNewVertex[VID];
			if (NewVID < 0)
			{
				NewVID = NewBoundaryVertices.Add(PlaneBoundaryVertices[VID]);
			}
			VID = NewVID;
		}
	}
	PlaneBoundaryVertices = MoveTemp(NewBoundaryVertices);
}



// Simpler invocation of CutWithPlanarCells w/ reasonable defaults
int32 CutWithPlanarCells(
	FPlanarCells& Cells,
	FGeometryCollection& Source,
	int32 TransformIdx,
	double Grout,
	double CollisionSampleSpacing,
	int32 RandomSeed,
	const TOptional<FTransform>& TransformCollection,
	bool bIncludeOutsideCellInOutput,
	bool bSetDefaultInternalMaterialsFromCollection,
	FProgressCancel* Progress,
	FVector CellsOrigin,
	FIslandSplitSettings IslandSplitSettings
)
{
	TArray<int32> TransformIndices { TransformIdx };
	return CutMultipleWithPlanarCells(Cells, Source, TransformIndices, Grout, CollisionSampleSpacing, RandomSeed, TransformCollection, bIncludeOutsideCellInOutput, bSetDefaultInternalMaterialsFromCollection, Progress, CellsOrigin, IslandSplitSettings);
}


int32 CutMultipleWithMultiplePlanes(
	const TArrayView<const FPlane>& Planes,
	FInternalSurfaceMaterials& InternalSurfaceMaterials,
	FGeometryCollection& Collection,
	const TArrayView<const int32>& TransformIndices,
	double Grout,
	double CollisionSampleSpacing,
	int32 RandomSeed,
	const TOptional<FTransform>& TransformCollection,
	bool bSetDefaultInternalMaterialsFromCollection,
	FProgressCancel* Progress,
	FIslandSplitSettings IslandSplitSettings
)
{
	FProgressCancel::FProgressScope PrepareScope = FProgressCancel::CreateScopeTo(Progress, .1f, LOCTEXT("CutWithMultiplePlanesInit", "Preparing to cut with planes"));
	int32 OrigNumGeom = Collection.FaceCount.Num();
	int32 CurNumGeom = OrigNumGeom;

	GeometryCollection::Facades::FCollectionMeshFacade CollectionMesh(Collection);
	if (!CollectionMesh.IsValid())
	{
		return -1;
	}

	if (bSetDefaultInternalMaterialsFromCollection)
	{
		InternalSurfaceMaterials.SetUVScaleFromCollection(CollectionMesh);
	}

	// Compute cuts in a local space where the collection has been translated to the origin, for more consistent noise + better LWC accuracy
	FTransform CollectionToWorld = TransformCollection.Get(FTransform::Identity);
	FTransform CollectionToWorldCentered;
	FVector Origin = PlanarCut_Locals::SeparateTranslation(CollectionToWorld, CollectionToWorldCentered);

	// Move planes to the same local space
	TArray<FPlane> CenteredPlanes;
	CenteredPlanes.Reserve(Planes.Num());
	for (const FPlane& Plane : Planes)
	{
		CenteredPlanes.Add(Plane.TranslateBy(-Origin));
	}

	FDynamicMeshCollection MeshCollection(&Collection, TransformIndices, CollectionToWorldCentered);

	PrepareScope.Done();
	if (Progress && Progress->Cancelled())
	{
		return -1;
	}
	FProgressCancel::FProgressScope CutScope = FProgressCancel::CreateScopeTo(Progress, .99f, LOCTEXT("CutWithMultiplePlanesBody", "Cutting with planes"));

	int32 NewGeomStartIdx = -1;
	NewGeomStartIdx = MeshCollection.CutWithMultiplePlanes(CenteredPlanes, Grout, CollisionSampleSpacing, IslandSplitSettings, RandomSeed, &Collection, InternalSurfaceMaterials, bSetDefaultInternalMaterialsFromCollection, Progress);

	CutScope.Done();
	if (Progress && Progress->Cancelled())
	{
		return -1;
	}
	FProgressCancel::FProgressScope ReindexScope = FProgressCancel::CreateScopeTo(Progress, 1);

	Collection.ReindexMaterials();

	ReindexScope.Done();

	return NewGeomStartIdx;
}

void CreateCuttingSurfacePreview(
	const FPlanarCells& Cells,
	const FBox& Bounds,
	double Grout,
	int32 RandomSeed,
	FDynamicMesh3& OutCuttingMeshes,
	TFunctionRef<bool(int)> FilterCellsFunc,
	const TOptional<FTransform>& TransformCollection,
	FProgressCancel* Progress,
	FVector CellsOrigin
)
{
	FTransform CollectionToWorld = TransformCollection.Get(FTransform::Identity);
	// Put Collection in the same local space as the Cells
	FTransform CollectionToWorldCentered = CollectionToWorld * FTransform(-CellsOrigin);

	FProgressCancel::FProgressScope SurfaceScope = FProgressCancel::CreateScopeTo(Progress, .99f);
	UE::Geometry::FAxisAlignedBox3d GeoBounds(Bounds);
	double OnePercentExtend = GeoBounds.MaxDim() * .01;
	FRandomStream RandomStream(RandomSeed);
	FCellMeshes CellMeshes(0, RandomStream, Cells, GeoBounds, Grout, OnePercentExtend, false);
	SurfaceScope.Done();

	if (Progress && Progress->Cancelled())
	{
		return;
	}

	FProgressCancel::FProgressScope AppendScope = FProgressCancel::CreateScopeTo(Progress, 1.0);
	OutCuttingMeshes.Clear();
	OutCuttingMeshes.EnableAttributes();
	FDynamicMeshEditor Editor(&OutCuttingMeshes);

	FMeshIndexMappings IndexMaps; // Needed for Editor.AppendMesh, not used otherwise
	for (int32 CellIdx = 0; CellIdx < CellMeshes.CellMeshes.Num(); ++CellIdx)
	{
		if (FilterCellsFunc(CellIdx))
		{
			Editor.AppendMesh(&CellMeshes.CellMeshes[CellIdx].AugMesh, IndexMaps);
		}
	}
	
	AppendScope.Done();
}

// Cut multiple Geometry groups inside a GeometryCollection with PlanarCells, and add each cut cell back to the GeometryCollection as a new child of their source Geometry
int32 CutMultipleWithPlanarCells(
	FPlanarCells& Cells,
	FGeometryCollection& Source,
	const TArrayView<const int32>& TransformIndices,
	double Grout,
	double CollisionSampleSpacing,
	int32 RandomSeed,
	const TOptional<FTransform>& TransformCollection,
	bool bIncludeOutsideCellInOutput,
	bool bSetDefaultInternalMaterialsFromCollection,
	FProgressCancel* Progress,
	FVector CellsOrigin,
	FIslandSplitSettings IslandSplitSettings
)
{
	FProgressCancel::FProgressScope CreateMeshCollectionScope = FProgressCancel::CreateScopeTo(Progress, .1f);
	if (bSetDefaultInternalMaterialsFromCollection)
	{
		Cells.InternalSurfaceMaterials.SetUVScaleFromCollection(Source);
	}

	FTransform CollectionToWorld = TransformCollection.Get(FTransform::Identity);
	// Put Collection in the same local space as the Cells
	FTransform CollectionToWorldCentered = CollectionToWorld * FTransform(-CellsOrigin);

	FDynamicMeshCollection MeshCollection(&Source, TransformIndices, CollectionToWorldCentered);
	CreateMeshCollectionScope.Done();

	if (Progress && Progress->Cancelled())
	{
		return -1;
	}

	FProgressCancel::FProgressScope CellMeshScope = FProgressCancel::CreateScopeTo(Progress, .1f);
	double OnePercentExtend = MeshCollection.Bounds.MaxDim() * .01;
	FRandomStream RandomStream(RandomSeed);
	FCellMeshes CellMeshes(Source.NumUVLayers(), RandomStream, Cells, MeshCollection.Bounds, Grout, OnePercentExtend, bIncludeOutsideCellInOutput);
	CellMeshScope.Done();

	if (Progress && Progress->Cancelled())
	{
		return -1;
	}

	FProgressCancel::FProgressScope CutScope = FProgressCancel::CreateScopeTo(Progress, .99f);
	int32 NewGeomStartIdx = -1;
	NewGeomStartIdx = MeshCollection.CutWithCellMeshes(Cells.InternalSurfaceMaterials, Cells.PlaneCells, CellMeshes, IslandSplitSettings, &Source, bSetDefaultInternalMaterialsFromCollection, CollisionSampleSpacing);
	CutScope.Done();

	if (Progress && Progress->Cancelled())
	{
		return -1;
	}

	FProgressCancel::FProgressScope ReindexScope = FProgressCancel::CreateScopeTo(Progress, 1);
	Source.ReindexMaterials();
	ReindexScope.Done();
	return NewGeomStartIdx;
}

int32 SplitIslands(
	FGeometryCollection& Collection,
	const TArrayView<const int32>& TransformIndices,
	double CollisionSampleSpacing,
	FProgressCancel* Progress,
	FIslandSplitSettings IslandSplitSettings
)
{
	FProgressCancel::FProgressScope CreateMeshCollectionScope = FProgressCancel::CreateScopeTo(Progress, .1f);

	FTransform CollectionToWorld = FTransform::Identity;

	FDynamicMeshCollection MeshCollection(&Collection, TransformIndices, CollectionToWorld);
	CreateMeshCollectionScope.Done();

	if (Progress && Progress->Cancelled())
	{
		return -1;
	}

	FProgressCancel::FProgressScope SplitScope = FProgressCancel::CreateScopeTo(Progress, .99f);
	int32 NewGeomStartIdx = -1;
	NewGeomStartIdx = MeshCollection.SplitAllIslands(&Collection, CollisionSampleSpacing, IslandSplitSettings);
	SplitScope.Done();

	if (Progress && Progress->Cancelled())
	{
		return -1;
	}

	FProgressCancel::FProgressScope ReindexScope = FProgressCancel::CreateScopeTo(Progress, 1);
	Collection.ReindexMaterials();
	ReindexScope.Done();
	return NewGeomStartIdx;
}

FDynamicMesh3 ConvertMeshDescriptionToCuttingDynamicMesh(const FMeshDescription* CuttingMesh, int32 NumUVLayers, FProgressCancel* Progress)
{
	// populate the BaseMesh with a conversion of the input mesh.
	FMeshDescriptionToDynamicMesh Converter;
	FDynamicMesh3 FullMesh; // full-featured conversion of the source mesh
	Converter.Convert(CuttingMesh, FullMesh, true);
	bool bHasInvalidNormals, bHasInvalidTangents;
	FStaticMeshOperations::HasInvalidVertexInstanceNormalsOrTangents(*CuttingMesh, bHasInvalidNormals, bHasInvalidTangents);
	if (bHasInvalidNormals || bHasInvalidTangents)
	{
		FDynamicMeshAttributeSet& Attribs = *FullMesh.Attributes();
		FDynamicMeshNormalOverlay* NTB[3]{ Attribs.PrimaryNormals(), Attribs.PrimaryTangents(), Attribs.PrimaryBiTangents() };
		if (bHasInvalidNormals)
		{
			FMeshNormals::InitializeOverlayToPerVertexNormals(NTB[0], false);
		}
		FMeshTangentsf Tangents(&FullMesh);
		Tangents.ComputeTriVertexTangents(NTB[0], Attribs.PrimaryUV(), { true, true });
		Tangents.CopyToOverlays(FullMesh);
	}

	if (Progress && Progress->Cancelled())
	{
		return FDynamicMesh3(); // return empty mesh on cancel
	}

	FDynamicMesh3 DynamicCuttingMesh; // version of mesh that is split apart at seams to be compatible w/ geometry collection, with corresponding attributes set
	SetGeometryCollectionAttributes(DynamicCuttingMesh, NumUVLayers);

	if (Progress && Progress->Cancelled())
	{
		return FDynamicMesh3(); // return empty mesh on cancel
	}

	// Note: This conversion will likely go away, b/c I plan to switch over to doing the boolean operations on the fuller rep, but the code can be adapted
	//		 to the dynamic mesh -> geometry collection conversion phase, as this same splitting will then need to happen there.
	if (ensure(FullMesh.HasAttributes() && FullMesh.Attributes()->NumUVLayers() >= 1 && FullMesh.Attributes()->NumNormalLayers() == 3))
	{
		if (!ensure(FullMesh.IsCompact()))
		{
			FullMesh.CompactInPlace();
		}
		// Triangles array is 1:1 with the input mesh
		TArray<FIndex3i> Triangles; Triangles.Init(FIndex3i::Invalid(), FullMesh.TriangleCount());
		
		FDynamicMesh3& OutMesh = DynamicCuttingMesh;
		FDynamicMeshAttributeSet& Attribs = *FullMesh.Attributes();
		FDynamicMeshNormalOverlay* NTB[3]{ Attribs.PrimaryNormals(), Attribs.PrimaryTangents(), Attribs.PrimaryBiTangents() };
		FDynamicMeshUVOverlay* UV = Attribs.PrimaryUV();
		TMap<FIndex4i, int> ElIDsToVID;
		int OrigMaxVID = FullMesh.MaxVertexID();
		for (int VID = 0; VID < OrigMaxVID; VID++)
		{
			check(FullMesh.IsVertex(VID));
			FVector3d Pos = FullMesh.GetVertex(VID);

			ElIDsToVID.Reset();
			FullMesh.EnumerateVertexTriangles(VID, [&FullMesh, &Triangles, &OutMesh, &NTB, &UV, &ElIDsToVID, Pos, VID, NumUVLayers](int32 TID)
			{
				FIndex3i InTri = FullMesh.GetTriangle(TID);
				int VOnT = IndexUtil::FindTriIndex(VID, InTri);
				FIndex4i ElIDs(
					NTB[0]->GetTriangle(TID)[VOnT],
					NTB[1]->GetTriangle(TID)[VOnT],
					NTB[2]->GetTriangle(TID)[VOnT],
					UV->GetTriangle(TID)[VOnT]);
				const int* FoundVID = ElIDsToVID.Find(ElIDs);

				FIndex3i& OutTri = Triangles[TID];
				if (FoundVID)
				{
					OutTri[VOnT] = *FoundVID;
				}
				else
				{
					FVector3f Normal = NTB[0]->GetElement(ElIDs.A);
					FVertexInfo Info(Pos, Normal, FVector3f(1, 1, 1));

					int OutVID = OutMesh.AppendVertex(Info);
					OutTri[VOnT] = OutVID;
					AugmentedDynamicMesh::SetTangent(OutMesh, OutVID, Normal, NTB[1]->GetElement(ElIDs.B), NTB[2]->GetElement(ElIDs.C));
					for (int32 UVLayerIdx = 0; UVLayerIdx < NumUVLayers; UVLayerIdx++)
					{
						AugmentedDynamicMesh::SetUV(OutMesh, OutVID, UV->GetElement(ElIDs.D), UVLayerIdx);
					}
					ElIDsToVID.Add(ElIDs, OutVID);
				}
			});
		}

		FDynamicMeshMaterialAttribute* OutMaterialID = OutMesh.Attributes()->GetMaterialID();
		for (int TID = 0; TID < Triangles.Num(); TID++)
		{
			FIndex3i& Tri = Triangles[TID];
			int AddedTID = OutMesh.AppendTriangle(Tri);
			if (ensure(AddedTID > -1))
			{
				OutMaterialID->SetValue(AddedTID, -1); // just use a single negative material ID by convention to indicate internal material
				AugmentedDynamicMesh::SetVisibility(OutMesh, AddedTID, true);
			}
		}
	}
	return DynamicCuttingMesh;
}

int32 CutWithMesh(
	const FDynamicMesh3& DynamicCuttingMesh,
	FTransform CuttingMeshTransform,
	FInternalSurfaceMaterials& InternalSurfaceMaterials,
	FGeometryCollection& Collection,
	const TArrayView<const int32>& TransformIndices,
	double CollisionSampleSpacing,
	const TOptional<FTransform>& TransformCollection,
	bool bSetDefaultInternalMaterialsFromCollection,
	FProgressCancel* Progress,
	FIslandSplitSettings IslandSplitSettings
)
{
	FProgressCancel::FProgressScope PrepareScope = FProgressCancel::CreateScopeTo(Progress, .1f);

	int32 NewGeomStartIdx = -1;

	if (bSetDefaultInternalMaterialsFromCollection)
	{
		InternalSurfaceMaterials.SetUVScaleFromCollection(Collection);
	}

	ensureMsgf(!InternalSurfaceMaterials.NoiseSettings.IsSet(), TEXT("Noise settings not yet supported for mesh-based fracture"));

	FTransform CollectionToWorld = TransformCollection.Get(FTransform::Identity);
	FTransform CollectionToWorldCentered;
	FVector Origin = PlanarCut_Locals::SeparateTranslation(CollectionToWorld, CollectionToWorldCentered);

	FTransform CuttingMeshTransformCentered = CuttingMeshTransform;
	CuttingMeshTransformCentered.SetTranslation(CuttingMeshTransform.GetTranslation() - Origin);

	if (Progress && Progress->Cancelled())
	{
		return -1;
	}

	FDynamicMeshCollection MeshCollection(&Collection, TransformIndices, CollectionToWorldCentered);
	int32 NumUVLayers = Collection.NumUVLayers();
	FCellMeshes CellMeshes(NumUVLayers, DynamicCuttingMesh, InternalSurfaceMaterials, CuttingMeshTransformCentered);

	TArray<TPair<int32, int32>> CellConnectivity;
	CellConnectivity.Add(TPair<int32, int32>(0, -1)); // there's only one 'inside' cell (0), so all cut surfaces are connecting the 'inside' cell (0) to the 'outside' cell (-1)

	PrepareScope.Done();
	if (Progress && Progress->Cancelled())
	{
		return -1;
	}
	FProgressCancel::FProgressScope CutScope = FProgressCancel::CreateScopeTo(Progress, .99f);

	NewGeomStartIdx = MeshCollection.CutWithCellMeshes(InternalSurfaceMaterials, CellConnectivity, CellMeshes, IslandSplitSettings, &Collection, bSetDefaultInternalMaterialsFromCollection, CollisionSampleSpacing);

	CutScope.Done();
	if (Progress && Progress->Cancelled())
	{
		return -1;
	}
	FProgressCancel::FProgressScope ReindexScope = FProgressCancel::CreateScopeTo(Progress, 1);

	Collection.ReindexMaterials();
	ReindexScope.Done();

	return NewGeomStartIdx;
}


void FindBoneVolumes(
	FGeometryCollection& Collection,
	const TArrayView<const int32>& TransformIndices,
	TArray<double>& OutVolumes,
	double ScalePerDimension,
	bool bIncludeClusters
)
{
	OutVolumes.Reset();

	// make sure volume attributes are up to date
	FGeometryCollectionConvexUtility::SetVolumeAttributes(&Collection);
	const TManagedArray<float>& VolumesAttrib = Collection.GetAttribute<float>("Volume", FTransformCollection::TransformGroup);

	TArray<FTransform> Transforms;
	TArray<int32> TransformIndicesArray(TransformIndices);
	if (TransformIndicesArray.Num() == 0)
	{
		for (int32 TransformIdx = 0; TransformIdx < Collection.TransformToGeometryIndex.Num(); TransformIdx++)
		{
			TransformIndicesArray.Add(TransformIdx);
		}
	}

	double VolumeScale = ScalePerDimension * ScalePerDimension * ScalePerDimension;

	OutVolumes.SetNum(TransformIndicesArray.Num());
	for (int32 Idx = 0; Idx < TransformIndicesArray.Num(); ++Idx)
	{
		int32 TransformIdx = TransformIndicesArray[Idx];
		if (bIncludeClusters || Collection.IsRigid(TransformIdx))
		{
			double UnscaledVolume = (double)VolumesAttrib[TransformIdx];
			OutVolumes[Idx] = UnscaledVolume * VolumeScale;
		}
		else
		{
			OutVolumes[Idx] = 0.0;
		}
	}
}


void FilterBonesByVolume(
	const FGeometryCollection& Collection,
	const TArrayView<const int32>& TransformIndices,
	const TArrayView<const double>& Volumes,
	TFunctionRef<bool(double Volume, int32 BoneIdx)> Filter,
	TArray<int32>& OutSmallBones,
	bool bIncludeClusters
)
{
	OutSmallBones.Reset();

	auto AddIdx = [&Collection, &Volumes, &Filter, &OutSmallBones, bIncludeClusters](int32 TransformIdx, double Volume)
	{
		constexpr int8 ClusterType = FGeometryCollection::ESimulationTypes::FST_Clustered;
		constexpr int8 RigidType = FGeometryCollection::ESimulationTypes::FST_Rigid;
		const int32 SimType = Collection.SimulationType[TransformIdx];
		const bool bCanConsiderIdx = (bIncludeClusters && SimType == ClusterType) || (SimType == RigidType && Collection.TransformToGeometryIndex[TransformIdx] > -1);
		if (bCanConsiderIdx && Filter(Volume, TransformIdx))
		{
			OutSmallBones.Add(TransformIdx);
		}
	};

	TArray<FTransform> Transforms;
	if (TransformIndices.Num() == 0)
	{
		int32 NumTransforms = Collection.TransformToGeometryIndex.Num();
		if (!ensure(Volumes.Num() == NumTransforms))
		{
			return;
		}
		for (int32 TransformIdx = 0; TransformIdx < NumTransforms; TransformIdx++)
		{
			AddIdx(TransformIdx, Volumes[TransformIdx]);
		}
	}
	else
	{
		if (!ensure(Volumes.Num() == TransformIndices.Num()))
		{
			return;
		}
		for (int32 Idx = 0; Idx < TransformIndices.Num(); ++Idx)
		{
			int32 TransformIdx = TransformIndices[Idx];
			AddIdx(TransformIdx, Volumes[Idx]);
		}
	}
}

// mesh adapter for a single geometry in a geometry collection
// Note: implicitly remaps vertices and faces to start at 0, to avoid wasting space in the Mesh AABB tree use case
struct FGeometryCollectionGeometryMeshAdapter
{
	TConstArrayView<FIntVector> FaceView;
	TConstArrayView<FVector3f> VertexView;
	int32 FaceStart, FaceNum, VertexStart, VertexNum;

	FGeometryCollectionGeometryMeshAdapter(const FGeometryCollection& Collection, int32 GeometryIdx)
	{
		FaceStart = Collection.FaceStart[GeometryIdx];
		FaceNum = Collection.FaceCount[GeometryIdx];
		VertexStart = Collection.VertexStart[GeometryIdx];
		VertexNum = Collection.VertexCount[GeometryIdx];
		FaceView = Collection.Indices.GetConstArray();
		VertexView = Collection.Vertex.GetConstArray();
	}

	bool IsTriangle(int32 TID) const
	{
		return TID >= 0 && TID < FaceNum;
	}
	bool IsVertex(int32 VID) const
	{
		return VID >= 0 && VID < VertexNum;
	}
	int32 MaxTriangleID() const
	{
		return FaceNum;
	}
	int32 MaxVertexID() const
	{
		return VertexNum;
	}
	int32 TriangleCount() const
	{
		return FaceNum;
	}
	int32 VertexCount() const
	{
		return VertexNum;
	}
	// note: GetChangeStamp is required by mesh aabb tree interface but not needed/useful otherwise
	uint64 GetChangeStamp() const
	{
		return 0;
	}
	FIndex3i GetTriangle(int32 TID) const
	{
		FIntVector Tri = FaceView[TID + FaceStart];
		return FIndex3i(Tri.X - VertexStart, Tri.Y - VertexStart, Tri.Z - VertexStart);
	}
	FVector3f GetVertex(int32 VID) const
	{
		return VertexView[VID + VertexStart];
	}

	template<typename VertexType>
	inline void GetTriVertices(int TID, VertexType& V0, VertexType& V1, VertexType& V2) const
	{
		FIntVector Tri = FaceView[TID + FaceStart];
		V0 = (VertexType)VertexView[Tri.X];
		V1 = (VertexType)VertexView[Tri.Y];
		V2 = (VertexType)VertexView[Tri.Z];
	}
};


void FindSmallBones(
	const FGeometryCollection& Collection,
	const TArrayView<const int32>& TransformIndices,
	const TArrayView<const double>& Volumes,
	double MinVolume,
	TArray<int32>& OutSmallBones,
	bool bIncludeClusters
)
{
	FilterBonesByVolume(Collection, TransformIndices, Volumes, 
		[MinVolume](double Volume, int32 BoneIdx) -> bool
		{
			return Volume < MinVolume;
		},
		OutSmallBones, bIncludeClusters);
}


static TMap<UE::Geometry::FIndex2i, float> ComputeGeometryVolumeContactWeightsFromProximity(
	const FGeometryCollection& Collection,
	const TArray<TSet<int32>>& GeomProximity,
	const TSet<int32>& GeomIdxSelection,
	const TMap<int32, double>* OptionalGeomToVol = nullptr,
	float DistanceThreshold = .01, float SamplesPerUnitArea = 10, int32 MaxSamplesPerTri = 100)
{
	TMap<UE::Geometry::FIndex2i, float> ContactWeights;
	TMultiMap<int32, int32> NeedConnections;
	for (int32 GeomIdx : GeomIdxSelection)
	{
		double GeomVol = OptionalGeomToVol ? (*OptionalGeomToVol)[GeomIdx] : 0.;
		for (int32 GeomNbrIdx : GeomProximity[GeomIdx])
		{
			// filter out contacts that we'll see twice
			if (GeomNbrIdx < GeomIdx && GeomIdxSelection.Contains(GeomNbrIdx))
			{
				continue;
			}

			// process in a consistent order. if we have volumes calculated, favor small->large,
			// since that may be a bit faster (typically fewer trianges/less surface area to sample)
			double NbrVol = OptionalGeomToVol ? (*OptionalGeomToVol)[GeomNbrIdx] : 0.;
			int32 GeomB = GeomIdx, GeomA = GeomNbrIdx;
			if (NbrVol < GeomVol || (NbrVol == GeomVol && GeomB < GeomA))
			{
				Swap(GeomA, GeomB);
			}
			NeedConnections.Add(GeomA, GeomB);
		}
	}
	TArray<int32> Keys;
	NeedConnections.GetKeys(Keys);
	for (int32 KeyGeom : Keys)
	{
		FGeometryCollectionGeometryMeshAdapter RefMesh(Collection, KeyGeom);
		TMeshAABBTree3<FGeometryCollectionGeometryMeshAdapter> RefSpatial(&RefMesh, true);
		FTransform3f RefTransform = (FTransform3f)GeometryCollectionAlgo::GlobalMatrix(Collection.Transform, Collection.Parent, Collection.TransformIndex[KeyGeom]);
		for (auto KeyItr = NeedConnections.CreateConstKeyIterator(KeyGeom); KeyItr; ++KeyItr)
		{
			int32 NbrGeom = KeyItr.Value();
			FTransform3f NbrTransform = (FTransform3f)GeometryCollectionAlgo::GlobalMatrix(Collection.Transform, Collection.Parent, Collection.TransformIndex[NbrGeom]);
			FGeometryCollectionGeometryMeshAdapter NbrMesh(Collection, NbrGeom);
			float ContactArea = UE::Geometry::TMeshQueries<FGeometryCollectionGeometryMeshAdapter>::ApproximateContactArea<TMeshAABBTree3<FGeometryCollectionGeometryMeshAdapter>, float>
				(NbrMesh, NbrTransform, RefSpatial, RefTransform, DistanceThreshold, SamplesPerUnitArea, MaxSamplesPerTri);
			FIndex2i OutputKey(KeyGeom, NbrGeom);
			OutputKey.Sort();
			ContactWeights.Add(OutputKey, ContactArea);
		}
	}
	return ContactWeights;
}

int32 MergeBones(
	FGeometryCollection& Collection,
	const TArrayView<const int32>& TransformIndicesView,
	const TArrayView<const double>& Volumes,
	double MinVolume,
	const TArrayView<const int32>& SmallTransformIndices,
	bool bUnionJoinedPieces,
	UE::PlanarCut::ENeighborSelectionMethod NeighborSelectionMethod,
	bool bUseCollectionProximity,
	bool bOnlySameParent
)
{
	FTransform CellsToWorld = FTransform::Identity;

	FGeometryCollectionProximityUtility ProximityUtility(&Collection);
	TArray<TSet<int32>> LocalProximity;
	const TArray<TSet<int32>>* UseProximity = nullptr;
	if (!bUseCollectionProximity)
	{
		LocalProximity = ProximityUtility.ComputePreciseProximity(Collection);
		UseProximity = &LocalProximity;
	}
	else
	{
		ProximityUtility.RequireProximity();
		UseProximity = &Collection.GetAttribute<TSet<int32>>("Proximity", FGeometryCollection::GeometryGroup).GetConstArray();
	}

	// local array so we can populate it with all transforms if input was empty
	TArray<int32> TransformIndices(TransformIndicesView);
	if (TransformIndices.Num() == 0)
	{
		for (int32 TransformIdx = 0; TransformIdx < Collection.TransformToGeometryIndex.Num(); TransformIdx++)
		{
			TransformIndices.Add(TransformIdx);
		}
	}
	if (!ensure(TransformIndices.Num() == Volumes.Num()))
	{
		return INDEX_NONE;
	}

	struct FRemoveGroup
	{
		int32 MergeTo = INDEX_NONE;
		double MergeTargetVolume = 0;
		double TotalVolume = 0;
		bool bRemoveMergeTarget = false;
		TArray<int32> ToRemove;

		FRemoveGroup() {}
		FRemoveGroup(const TMap<int32, double>& GeomVolMaps, double MinVolume, int32 SmallIdx, int32 BigIdx)
		{
			ToRemove.Add(SmallIdx);
			double BigVol = GeomVolMaps[BigIdx];
			if (BigVol < MinVolume)
			{
				ToRemove.Add(BigIdx);
				bRemoveMergeTarget = true;
			}
			MergeTargetVolume = BigVol;
			TotalVolume = BigVol + GeomVolMaps[SmallIdx];
			MergeTo = BigIdx;
		}

		bool IsValid()
		{
			return MergeTo != INDEX_NONE;
		}

		void UpdateMergeTarget(int Idx, double Volume)
		{
			if (bRemoveMergeTarget)
			{
				if (Volume > MergeTargetVolume)
				{
					MergeTo = Idx;
					MergeTargetVolume = Volume;
				}
			}
		}

		// add a too-small geometry to this group
		void AddSmall(const TMap<int32, double>& GeomVolMaps, int32 SmallIdx)
		{
			double SmallVol = GeomVolMaps[SmallIdx];
			TotalVolume += SmallVol;
			UpdateMergeTarget(SmallIdx, SmallVol);
			checkSlow(!ToRemove.Contains(SmallIdx));
			ToRemove.Add(SmallIdx);
		}

		// add a neighbor of a too-small geometry to an existing group
		void AddBig(const TMap<int32, double>& GeomVolMaps, double MinVolume, int32 BigIdx)
		{
			double BigVol = GeomVolMaps[BigIdx];
			TotalVolume += BigVol;
			UpdateMergeTarget(BigIdx, BigVol);
			if (BigVol < MinVolume)
			{
				checkSlow(!ToRemove.Contains(BigIdx));
				ToRemove.Add(BigIdx);
			}
			else
			{
				bRemoveMergeTarget = false;
			}
		}

		void TransferGroup(FRemoveGroup& SmallGroup, TMap<int32, int32>& GeomIdxToRemoveGroupIdx, int32 NewIdx)
		{
			for (int32 RmIdx : SmallGroup.ToRemove)
			{
				checkSlow(!ToRemove.Contains(RmIdx));
				ToRemove.Add(RmIdx);
				GeomIdxToRemoveGroupIdx[RmIdx] = NewIdx;
			}
			if (!SmallGroup.bRemoveMergeTarget)
			{
				checkSlow(!ToRemove.Contains(SmallGroup.MergeTo));
				ToRemove.Add(SmallGroup.MergeTo);
				GeomIdxToRemoveGroupIdx[SmallGroup.MergeTo] = NewIdx;
			}
			TotalVolume += SmallGroup.TotalVolume;
			SmallGroup = FRemoveGroup(); // clear old group
		}

		bool IsGroupSmall(double MinVolume)
		{
			return TotalVolume < MinVolume;
		}
	};
	TMap<int32, double> GeomToVol;
	TMap<int32, int32> GeomToParent;
	TMap<int32, int32> GeomIdxToRemoveGroupIdx;
	TArray<FRemoveGroup> RemoveGroups;
	TSet<int32> TooSmalls;
	TSet<int32> CanMerge;

	// GeomToCenter is just a cache for GetCenter; may not be worth caching as long as 'center' == bounding box center
	// TODO: consider switching to a more accurate 'center' and/or removing the cache
	TMap<int32, FVector> GeomToCenter;
	auto GetCenter = [&Collection, &GeomToCenter](int32 GeomIdx) -> FVector
	{
		FVector* CachedCenter = GeomToCenter.Find(GeomIdx);
		if (CachedCenter)
		{
			return *CachedCenter;
		}
		int32 TransformIdx = Collection.TransformIndex[GeomIdx];
		FTransform Transform = GeometryCollectionAlgo::GlobalMatrix(Collection.Transform, Collection.Parent, TransformIdx);
		FVector Center = Transform.TransformPosition(Collection.BoundingBox[GeomIdx].GetCenter());
		GeomToCenter.Add(GeomIdx, Center);
		return Center;
	};

	for (int32 Idx = 0; Idx < TransformIndices.Num(); ++Idx)
	{
		int32 TransformIdx = TransformIndices[Idx];
		int32 GeomIdx = Collection.TransformToGeometryIndex[TransformIdx];
		if (GeomIdx > -1 && Collection.IsRigid(TransformIdx))
		{
			CanMerge.Add(GeomIdx);
			GeomToVol.Add(GeomIdx, Volumes[Idx]);
			GeomToParent.Add(GeomIdx, Collection.Parent[TransformIdx]);
		}
	}
	for (int32 TransformIdx : SmallTransformIndices)
	{
		int32 GeomIdx = Collection.TransformToGeometryIndex[TransformIdx];
		if (GeomIdx > -1 && Collection.IsRigid(TransformIdx) && GeomToVol.Contains(GeomIdx))
		{
			TooSmalls.Add(GeomIdx);
		}
		else
		{
			ensureMsgf(false, TEXT("Cannot merge bones that have no geometry attached"));
		}
	}

	TMap<UE::Geometry::FIndex2i, float> ConnectionWeights;
	if (NeighborSelectionMethod == ENeighborSelectionMethod::LargestContactArea)
	{
		ConnectionWeights = ComputeGeometryVolumeContactWeightsFromProximity(Collection, *UseProximity, TooSmalls, &GeomToVol);
	}


	for (int32 SmallIdx : TooSmalls)
	{
		int32* SmallRemoveGroupIdx = GeomIdxToRemoveGroupIdx.Find(SmallIdx);
		if (SmallRemoveGroupIdx)
		{
			if (RemoveGroups[*SmallRemoveGroupIdx].TotalVolume >= MinVolume)
			{
				continue;
			}
		}

		const TSet<int32>& Prox = (*UseProximity)[SmallIdx];
		double BestScore = -FMathd::MaxReal;
		int32 BestNbrIdx = INDEX_NONE;
		for (int32 NbrIdx : Prox)
		{
			if (NbrIdx != SmallIdx && CanMerge.Contains(NbrIdx))
			{
				if (bOnlySameParent && GeomToParent[NbrIdx] != GeomToParent[SmallIdx])
				{
					continue;
				}
				double Score;
				if (NeighborSelectionMethod == UE::PlanarCut::ENeighborSelectionMethod::LargestNeighbor)
				{
					Score = GeomToVol[NbrIdx];
				}
				else if (NeighborSelectionMethod == UE::PlanarCut::ENeighborSelectionMethod::NearestCenter)
				{
					Score = 1.0 / (DOUBLE_SMALL_NUMBER + DistanceSquared(GetCenter(NbrIdx), GetCenter(SmallIdx)));
				}
				else // use pre-computed connection weight map
				{
					FIndex2i Key(SmallIdx, NbrIdx);
					Key.Sort();
					float* FoundWt = ConnectionWeights.Find(Key);
					Score = FoundWt ? *FoundWt : 0.f;
				}
				if (Score > BestScore)
				{
					BestScore = Score;
					BestNbrIdx = NbrIdx;
				}
			}
		}
		if (BestNbrIdx == INDEX_NONE)
		{
			UE_LOGF(LogPlanarCut, Warning, "Couldn't fix Bone %d: No neighbors found in proximity graph", SmallIdx);
			continue;
		}

		if (SmallRemoveGroupIdx)
		{
			int32 OldSGIdx = *SmallRemoveGroupIdx;
			int32* BigRemoveGroupIdx = GeomIdxToRemoveGroupIdx.Find(BestNbrIdx);
			if (BigRemoveGroupIdx)
			{
				int32 BigRGIdx = *BigRemoveGroupIdx;
				if (OldSGIdx != BigRGIdx)
				{
					RemoveGroups[BigRGIdx].TransferGroup(RemoveGroups[OldSGIdx], GeomIdxToRemoveGroupIdx, BigRGIdx);
					checkSlow(GeomIdxToRemoveGroupIdx.FindKey(OldSGIdx) == nullptr);
				}
			}
			else
			{
				RemoveGroups[OldSGIdx].AddBig(GeomToVol, MinVolume, BestNbrIdx);
				checkSlow(!GeomIdxToRemoveGroupIdx.Contains(BestNbrIdx));
				GeomIdxToRemoveGroupIdx.Add(BestNbrIdx, OldSGIdx);
			}
		}
		else
		{
			int32* BigRemoveGroupIdx = GeomIdxToRemoveGroupIdx.Find(BestNbrIdx);
			if (BigRemoveGroupIdx)
			{
				int32 BigRGIdx = *BigRemoveGroupIdx;
				RemoveGroups[BigRGIdx].AddSmall(GeomToVol, SmallIdx);
				checkSlow(!GeomIdxToRemoveGroupIdx.Contains(SmallIdx));
				GeomIdxToRemoveGroupIdx.Add(SmallIdx, BigRGIdx);
			}
			else
			{
				int32 RemoveGroupIdx = RemoveGroups.Emplace(GeomToVol, MinVolume, SmallIdx, BestNbrIdx);
				checkSlow(!GeomIdxToRemoveGroupIdx.Contains(SmallIdx));
				checkSlow(!GeomIdxToRemoveGroupIdx.Contains(BestNbrIdx));
				GeomIdxToRemoveGroupIdx.Add(SmallIdx, RemoveGroupIdx);
				GeomIdxToRemoveGroupIdx.Add(BestNbrIdx, RemoveGroupIdx);
			}
		}
	}

	TArray<int32> AllRemoveIndices;
	TArray<int32> AllUpdateIndices;

	for (FRemoveGroup& Group : RemoveGroups)
	{
		if (!Group.IsValid())
		{
			continue;
		}
		if (Group.bRemoveMergeTarget)
		{
			Group.ToRemove.RemoveSingle(Group.MergeTo);
		}
		for (int32 RmIdx : Group.ToRemove)
		{
			AllRemoveIndices.Add(Collection.TransformIndex[RmIdx]);
		}
		AllUpdateIndices.Add(Collection.TransformIndex[Group.MergeTo]);
	}

	AllRemoveIndices.Sort();
	AllUpdateIndices.Sort();
	
	FDynamicMeshCollection RemoveCollection(&Collection, AllRemoveIndices, CellsToWorld, true);
	FDynamicMeshCollection UpdateCollection(&Collection, AllUpdateIndices, CellsToWorld, true);

	TMap<int32, int32> GeoIdxToRmMeshIdx;
	for (int32 RmMeshIdx = 0; RmMeshIdx < RemoveCollection.Meshes.Num(); RmMeshIdx++)
	{
		int32 TransformIdx = RemoveCollection.Meshes[RmMeshIdx].TransformIndex;
		GeoIdxToRmMeshIdx.Add(
			Collection.TransformToGeometryIndex[TransformIdx],
			RmMeshIdx
		);
	}

	using FMeshData = UE::PlanarCut::FDynamicMeshCollection::FMeshData;

	for (int32 UpMeshIdx = 0; UpMeshIdx < UpdateCollection.Meshes.Num(); UpMeshIdx++)
	{
		FMeshData& UpMeshData = UpdateCollection.Meshes[UpMeshIdx];
		int32 UpGeoIdx = Collection.TransformToGeometryIndex[UpMeshData.TransformIndex];
		FRemoveGroup& Group = RemoveGroups[GeomIdxToRemoveGroupIdx[UpGeoIdx]];
		if (!ensure(Group.IsValid()))
		{
			continue;
		}
		FDynamicMeshEditor MeshEditor(&UpMeshData.AugMesh);
		for (int32 RmGeoIdx : Group.ToRemove)
		{
			FMeshData& RmMeshData = RemoveCollection.Meshes[GeoIdxToRmMeshIdx[RmGeoIdx]];
			if (bUnionJoinedPieces)
			{
				FMeshBoolean Boolean(&UpMeshData.AugMesh, &RmMeshData.AugMesh, &UpMeshData.AugMesh, FMeshBoolean::EBooleanOp::Union);
				Boolean.bWeldSharedEdges = false;
				Boolean.bSimplifyAlongNewEdges = true;
				Boolean.Compute();
			}
			else
			{
				FMeshIndexMappings IndexMaps_Unused;
				MeshEditor.AppendMesh(&RmMeshData.AugMesh, IndexMaps_Unused);
			}
		}
	}

	UpdateCollection.UpdateAllCollections(Collection);

	for (FRemoveGroup& Group : RemoveGroups)
	{
		if (!Group.IsValid())
		{
			continue;
		}
		int32 MergeTransformIdx = Collection.TransformIndex[Group.MergeTo];
		TArray<int32> Children;
		for (int32 RmIdx : Group.ToRemove)
		{
			int32 RmTransformIdx = Collection.TransformIndex[RmIdx];
			for (int32 ChildIdx : Collection.Children[RmTransformIdx])
			{
				Children.Add(ChildIdx);
			}
		}
		if (Children.Num())
		{
			GeometryCollectionAlgo::ParentTransforms(&Collection, MergeTransformIdx, Children);
		}
	}

	// remove transforms for all geometry that was merged in
	FManagedArrayCollection::FProcessingParameters ProcessingParams;
#if !UE_BUILD_DEBUG
	ProcessingParams.bDoValidation = false;
#endif
	Collection.RemoveElements(FGeometryCollection::TransformGroup, AllRemoveIndices, ProcessingParams);

	return INDEX_NONE; // TODO: consider tracking smallest index of updated groups?  but no reason to do so currently
}

namespace
{
	static double GetClusterVolumeFromRigidsHelper(const FGeometryCollection& Collection, const TArrayView<const double>& Volumes, int32 BoneIdx)
	{
		if (Collection.SimulationType[BoneIdx] == FGeometryCollection::ESimulationTypes::FST_Rigid)
		{
			return Volumes[BoneIdx];
		}
		double Sum = 0;
		for (int32 Child : Collection.Children[BoneIdx])
		{
			Sum += GetClusterVolumeFromRigidsHelper(Collection, Volumes, Child);
		}
		return Sum;
	}

	static FBox GetClusterBoundsFromRigidsHelper(const FGeometryCollection& Collection, const TArray<FBox>& GeoBoxesInWorldSpace, int32 BoneIdx)
	{
		int32 GeoIdx = Collection.TransformToGeometryIndex[BoneIdx];
		if (GeoIdx != INDEX_NONE)
		{
			return GeoBoxesInWorldSpace[GeoIdx];
		}
		FBox Combined(ForceInit);
		for (int32 Child : Collection.Children[BoneIdx])
		{
			Combined += GetClusterBoundsFromRigidsHelper(Collection, GeoBoxesInWorldSpace, Child);
		}
		return Combined;
	}
	
	static void GetRigidGeometrySelectionFromBoneSelection(const FGeometryCollection& Collection, int32 BoneIdx, TSet<int32>& OutGeometryIndices)
	{
		int32 GeoIdx = Collection.TransformToGeometryIndex[BoneIdx];
		if (Collection.IsRigid(BoneIdx) && GeoIdx != INDEX_NONE)
		{
			OutGeometryIndices.Add(GeoIdx);
		}
		else if (Collection.IsClustered(BoneIdx))
		{
			for (int32 Child : Collection.Children[BoneIdx])
			{
				GetRigidGeometrySelectionFromBoneSelection(Collection, Child, OutGeometryIndices);
			}
		}
	}
}

void MergeClusters(
	FGeometryCollection& Collection,
	const TArrayView<const double>& Volumes,
	double MinVolume,
	const TArrayView<const int32>& SmallTransformIndices,
	UE::PlanarCut::ENeighborSelectionMethod NeighborSelectionMethod,
	bool bOnlyMergeInProximity,
	bool bOnlySameParent,
	bool bUseCollectionProximity
)
{
	if (Collection.NumElements(FGeometryCollection::TransformGroup) == 0)
	{
		// nothing to do, early exit
		return;
	}

	FGeometryCollectionProximityUtility ProximityUtility(&Collection);
	TArray<TSet<int32>> LocalProximity;
	const TArray<TSet<int32>>* UseProximity = nullptr;
	if (!bUseCollectionProximity)
	{
		LocalProximity = ProximityUtility.ComputePreciseProximity(Collection);
		UseProximity = &LocalProximity;
	}
	else
	{
		ProximityUtility.RequireProximity();
		UseProximity = &Collection.GetAttribute<TSet<int32>>("Proximity", FGeometryCollection::GeometryGroup).GetConstArray();
	}
	const TManagedArray<int32>* Level = Collection.FindAttribute<int32>("Level", FGeometryCollection::TransformGroup);
	if (!ensure(Level != nullptr))
	{
		// cluster merging requires a level attribute
		return;
	}

	TArray<FBox> GeoBoxesInWorldSpace;
	TArray<FTransform> GlobalTransforms;
	GeometryCollectionAlgo::GlobalMatrices(Collection.Transform, Collection.Parent, GlobalTransforms);
	GeoBoxesInWorldSpace.SetNum(Collection.TransformIndex.Num());
	for (int32 GeoIdx = 0; GeoIdx < GeoBoxesInWorldSpace.Num(); ++GeoIdx)
	{
		FTransform ToWorld = GlobalTransforms[Collection.TransformIndex[GeoIdx]];
		FBox GeoBox(ForceInit);
		int32 VStart = Collection.VertexStart[GeoIdx];
		int32 VEnd = VStart + Collection.VertexCount[GeoIdx];
		for (int32 VIdx = VStart; VIdx < VEnd; ++VIdx)
		{
			GeoBox += ToWorld.TransformPosition((FVector)Collection.Vertex[VIdx]);
		}
		GeoBoxesInWorldSpace[GeoIdx] = GeoBox;
	}

	// helper to grab same level neighbors from the proximity info + traversal
	// only selects rigid or cluster bones
	// Note we do the traversal each time rather using a pre-computed connection graph because the cluster neighbors will change as we merge clusters
	// (the Proximity attribute should be stable because it is on the geometry group, not the transform group)
	auto GetSameLevelNeighbors = [&Collection, &UseProximity, &Level, bOnlySameParent](int32 BoneIdx, TSet<int32>& OutNeighbors) -> void
	{
		int32 TargetLevel = (*Level)[BoneIdx];
		TArray<int32> ToProcess;
		ToProcess.Add(BoneIdx);
		int32 BoneParent = Collection.Parent[BoneIdx];
		while (!ToProcess.IsEmpty())
		{
			int32 ProcessIdx = ToProcess.Pop();
			// we're processing a rigid node: access the proximity graph and look for neighbor nodes
			if (Collection.SimulationType[ProcessIdx] == FGeometryCollection::ESimulationTypes::FST_Rigid
				&& Collection.TransformToGeometryIndex[ProcessIdx] != INDEX_NONE)
			{
				for (int32 NbrGeo : (*UseProximity)[Collection.TransformToGeometryIndex[ProcessIdx]])
				{
					// Traverse parents to find a same-level cluster on this neighbor
					int32 TraverseParent = Collection.TransformIndex[NbrGeo];
					while (TraverseParent != -1 && (*Level)[TraverseParent] > TargetLevel)
					{
						TraverseParent = Collection.Parent[TraverseParent];
					}
					if (TraverseParent == -1) // failed to find a same-level neighbor
					{
						continue;
					}
					// any different, same-level cluster is a valid neighbor to consider
					bool bIsRigid = Collection.SimulationType[TraverseParent] == FGeometryCollection::ESimulationTypes::FST_Rigid;
					// A valid cluster is one that has not been emptied by a previous merge (though we should not reach such an empty cluster in this traversal)
					bool bIsValidCluster = !Collection.Children[TraverseParent].IsEmpty() && Collection.SimulationType[TraverseParent] == FGeometryCollection::ESimulationTypes::FST_Clustered;
					bool bValidParent = !bOnlySameParent || Collection.Parent[TraverseParent] == BoneParent;
					if (TraverseParent != BoneIdx && TraverseParent != -1 && (*Level)[TraverseParent] == TargetLevel && bValidParent
						&& (bIsRigid || bIsValidCluster)) // only merge to rigid or cluster bones
					{
						OutNeighbors.Add(TraverseParent);
					}
				}
			}
			// we're processing a cluster node: traverse down to children to find a rigid node with proximity info
			else if (Collection.SimulationType[ProcessIdx] == FGeometryCollection::ESimulationTypes::FST_Clustered)
			{
				ToProcess.Reserve(ToProcess.Num() + Collection.Children[ProcessIdx].Num());
				for (int32 ChildIdx : Collection.Children[ProcessIdx])
				{
					ToProcess.Add(ChildIdx);
				}
			}
		}
	};
	
	TSet<int32> ToProcessGeometry;
	for (int32 BoneIdx : SmallTransformIndices)
	{
		GetRigidGeometrySelectionFromBoneSelection(Collection, BoneIdx, ToProcessGeometry);
	}
	TMap<UE::Geometry::FIndex2i, float> ConnectionWeights;
	if (NeighborSelectionMethod == ENeighborSelectionMethod::LargestContactArea)
	{
		ConnectionWeights = ComputeGeometryVolumeContactWeightsFromProximity(Collection, *UseProximity, ToProcessGeometry);
	}

	TSet<int32> ReconsiderIfSmall, DoNotReconsider; // Sets to track nodes we've already merged, and may not need to re-merge
	TSet<int32> Neighbors; // Set to be re-used in the below loop, tracking neighbors to consider
	TArray<int32> ChildNodes; // Array to be re-used in the below loop, tracking nodes to merge under a new parent
	TArray<int32> ToProcess(SmallTransformIndices); // Need a mutable copy of SmallTransformIndices to update the array with merged nodes
	for (int32 ProcessIdx = 0; ProcessIdx < ToProcess.Num(); ++ProcessIdx)
	{
		int32 MergeIdx = ToProcess[ProcessIdx];
		if (Collection.Parent[MergeIdx] == -1 || DoNotReconsider.Contains(MergeIdx)) // can't merge root nodes
		{
			continue;
		}
		bool bIsCluster = Collection.SimulationType[MergeIdx] == FGeometryCollection::ESimulationTypes::FST_Clustered;
		if (ReconsiderIfSmall.Contains(MergeIdx))
		{
			double Volume = GetClusterVolumeFromRigidsHelper(Collection, Volumes, MergeIdx);
			if (Volume > MinVolume)
			{
				continue;
			}
		}
		Neighbors.Reset();
		GetSameLevelNeighbors(MergeIdx, Neighbors);
		int32 BestNeighbor = INDEX_NONE;
		if (Neighbors.IsEmpty() && !bOnlyMergeInProximity)
		{
			FVector Center = GetClusterBoundsFromRigidsHelper(Collection, GeoBoxesInWorldSpace, MergeIdx).GetCenter();
			double ClosestDistSq = FMathd::MaxReal;
			for (int32 TransformIdx = 0; TransformIdx < Collection.Transform.Num(); ++TransformIdx)
			{
				if (TransformIdx == MergeIdx || (*Level)[MergeIdx] != (*Level)[TransformIdx]
					|| Collection.SimulationType[TransformIdx] != FGeometryCollection::ESimulationTypes::FST_Clustered)
				{
					continue;
				}
				FVector OtherCenter = GetClusterBoundsFromRigidsHelper(Collection, GeoBoxesInWorldSpace, TransformIdx).GetCenter();
				double DistSq = FVector::DistSquared(Center, OtherCenter);
				if (BestNeighbor == INDEX_NONE || DistSq < ClosestDistSq)
				{
					BestNeighbor = TransformIdx;
					ClosestDistSq = DistSq;
				}
			}
		}
		else
		{
			if (NeighborSelectionMethod == ENeighborSelectionMethod::NearestCenter)
			{
				FVector Center = GetClusterBoundsFromRigidsHelper(Collection, GeoBoxesInWorldSpace, MergeIdx).GetCenter();
				double ClosestDistSq = FMathd::MaxReal;
				for (int32 NeighborIdx : Neighbors)
				{
					FVector OtherCenter = GetClusterBoundsFromRigidsHelper(Collection, GeoBoxesInWorldSpace, NeighborIdx).GetCenter();
					double DistSq = FVector::DistSquared(Center, OtherCenter);
					if (BestNeighbor == INDEX_NONE || DistSq < ClosestDistSq)
					{
						BestNeighbor = NeighborIdx;
						ClosestDistSq = DistSq;
					}
				}
			}
			else if (NeighborSelectionMethod == ENeighborSelectionMethod::LargestNeighbor)
			{
				double Volume = GetClusterVolumeFromRigidsHelper(Collection, Volumes, MergeIdx);
				double LargestVolume = -1;
				for (int32 NeighborIdx : Neighbors)
				{
					double NbrVolume = GetClusterVolumeFromRigidsHelper(Collection, Volumes, NeighborIdx);
					if (BestNeighbor == INDEX_NONE || NbrVolume > LargestVolume)
					{
						BestNeighbor = NeighborIdx;
						LargestVolume = NbrVolume;
					}
				}
			}
			else // ENeighborSelectionMethod::LargestContactArea
			{
				TSet<int32> MergeGeo, NbrGeo;
				GetRigidGeometrySelectionFromBoneSelection(Collection, MergeIdx, MergeGeo);
				double LargestContact = -1;
				for (int32 NeighborIdx : Neighbors)
				{
					double ContactArea = 0.;
					NbrGeo.Reset();
					GetRigidGeometrySelectionFromBoneSelection(Collection, NeighborIdx, NbrGeo);
					// Look for proximity-neighbors of the merge bone's geometry
					// that belong to the current neighbor bone's geometry,
					// and add the corresponding contact area
					for (int32 MergeGeoIdx : MergeGeo)
					{
						for (int32 MergeGeoNbrIdx : (*UseProximity)[MergeGeoIdx])
						{
							if (NbrGeo.Contains(MergeGeoNbrIdx))
							{
								FIndex2i Key(MergeGeoIdx, MergeGeoNbrIdx);
								Key.Sort();
								float* FoundArea = ConnectionWeights.Find(Key);
								if (FoundArea)
								{
									ContactArea += double(*FoundArea);
								}
							}
						}
					}
					if (BestNeighbor == INDEX_NONE || ContactArea > LargestContact)
					{
						BestNeighbor = NeighborIdx;
						LargestContact = ContactArea;
					}
				}
			}
		}
		if (BestNeighbor == INDEX_NONE)
		{
			continue;
		}
		
		int32 ToMerge = MergeIdx;
		if (Collection.SimulationType[BestNeighbor] != FGeometryCollection::ESimulationTypes::FST_Clustered)
		{
			if (Collection.SimulationType[ToMerge] == FGeometryCollection::ESimulationTypes::FST_Clustered)
			{
				Swap(ToMerge, BestNeighbor);
			}
			else
			{
				// both nodes are rigid: need to create a new cluster for them
				ChildNodes.Reset();
				ChildNodes.Add(MergeIdx);
				ChildNodes.Add(BestNeighbor);
				int32 NewCluster = FGeometryCollectionClusteringUtility::ClusterBonesUnderNewNode(&Collection, MergeIdx, ChildNodes, true, false);
				// Now that the node is clustered at a lower level, we shouldn't try to re-merge it with any other nodes
				DoNotReconsider.Add(BestNeighbor);
				double Volume = GetClusterVolumeFromRigidsHelper(Collection, Volumes, NewCluster);
				if (Volume < MinVolume)
				{
					// if the new cluster is still small, we may want to merge it again
					ToProcess[ProcessIdx] = NewCluster;
					ProcessIdx--; // this re-considers ProcessIdx on the next iteration
				}
				continue;
			}
		}

		// From the if / swap above, BestNeighbor must now be a cluster node, so we can merge under it
		checkSlow(Collection.SimulationType[BestNeighbor] == FGeometryCollection::ESimulationTypes::FST_Clustered);
		int32 ParentNode = BestNeighbor;
		ChildNodes.Reset();
		if (!bIsCluster)
		{
			ChildNodes.Add(ToMerge);
		}
		else
		{
			ChildNodes.Append(Collection.Children[ToMerge].Array());
		}
		// Note: This updates the Level, Children and Parent attributes
		FGeometryCollectionClusteringUtility::ClusterBonesUnderExistingNode(&Collection, ParentNode, ChildNodes);
		ReconsiderIfSmall.Add(BestNeighbor);
	}

	FGeometryCollectionClusteringUtility::RemoveDanglingClusters(&Collection);
	FGeometryCollectionClusteringUtility::RemoveClustersOfOnlyOneChild(&Collection);
}

namespace
{
	void AddAllChildrenExceptEmbedded(FGeometryCollection& Collection, TSet<int32>& NodesSet, int32 Node)
	{
		if (Collection.SimulationType[Node] != FGeometryCollection::ESimulationTypes::FST_None)
		{
			NodesSet.Add(Node);
			for (int32 Child : Collection.Children[Node])
			{
				AddAllChildrenExceptEmbedded(Collection, NodesSet, Child);
			}
		}
	}
}

void MergeAllSelectedBones(
	FGeometryCollection& Collection,
	const TArrayView<const int32>& TransformIndices,
	bool bUnionJoinedPieces
)
{
	if (TransformIndices.IsEmpty())
	{
		return;
	}

	if (!Collection.HasAttribute("Level", FGeometryCollection::TransformGroup))
	{
		FGeometryCollectionClusteringUtility::UpdateHierarchyLevelOfChildren(&Collection, -1);
	}
	const TManagedArray<int32>& Level = Collection.GetAttribute<int32>("Level", FGeometryCollection::TransformGroup);
	int32 TopLevelNode = TransformIndices[0];
	int32 TopLevel = Level[TopLevelNode];

	TSet<int32> AllNodesSet;
	for (int32 Node : TransformIndices)
	{
		if (Level[Node] < TopLevel)
		{
			TopLevel = Level[Node];
			TopLevelNode = Node;
		}
		AddAllChildrenExceptEmbedded(Collection, AllNodesSet, Node);
	}
	TArray<int32> AllNodes = AllNodesSet.Array();

	if (AllNodes.Num() < 2)
	{
		return; // not enough pieces to need any merging
	}

	TArray<int32> AllUpdateIndices, AllRemoveIndices;
	AllUpdateIndices.Add(TopLevelNode);
	AllRemoveIndices.Reserve(AllNodes.Num() - 1);
	for (int32 Node : AllNodes)
	{
		if (Node != TopLevelNode)
		{
			AllRemoveIndices.Add(Node);
		}
	}


	AllRemoveIndices.Sort();
	AllUpdateIndices.Sort();

	FTransform CellsToWorld = FTransform::Identity;

	FDynamicMeshCollection RemoveCollection(&Collection, AllRemoveIndices, CellsToWorld, true);
	FDynamicMeshCollection UpdateCollection(&Collection, AllUpdateIndices, CellsToWorld, true);

	using FMeshData = UE::PlanarCut::FDynamicMeshCollection::FMeshData;
	bool bNeedAddGeometry = false;
	if (UpdateCollection.Meshes.IsEmpty())
	{
		int32 NumUVLayers = Collection.NumUVLayers();
		FMeshData* MeshData = new FMeshData(NumUVLayers);
		MeshData->FromCollection = GeometryCollectionAlgo::GlobalMatrix(Collection.Transform, Collection.Parent, TopLevelNode);
		MeshData->TransformIndex = TopLevelNode;
		UpdateCollection.Meshes.Add(MeshData);

		bNeedAddGeometry = true;
	}
	check(UpdateCollection.Meshes.Num() == 1);
	FMeshData& UpMeshData = UpdateCollection.Meshes[0];
	FDynamicMeshEditor MeshEditor(&UpMeshData.AugMesh);
	for (int32 RmMeshIdx = 0; RmMeshIdx < RemoveCollection.Meshes.Num(); RmMeshIdx++)
	{
		FMeshData& RmMeshData = RemoveCollection.Meshes[RmMeshIdx];
		if (bUnionJoinedPieces)
		{
			FMeshBoolean Boolean(&UpMeshData.AugMesh, &RmMeshData.AugMesh, &UpMeshData.AugMesh, FMeshBoolean::EBooleanOp::Union);
			Boolean.bWeldSharedEdges = false;
			Boolean.bSimplifyAlongNewEdges = true;
			Boolean.Compute();
		}
		else
		{
			FMeshIndexMappings IndexMaps_Unused;
			MeshEditor.AppendMesh(&RmMeshData.AugMesh, IndexMaps_Unused);
		}
	}

	if (UpMeshData.AugMesh.TriangleCount() > 0)
	{
		if (bNeedAddGeometry)
		{
			if (Collection.TransformToGeometryIndex[TopLevelNode] == -1)
			{
				// Create a new geometry element to fill w/ the merged geometry
				int32 GeometryIdx = Collection.AddElements(1, FGeometryCollection::GeometryGroup);
				Collection.TransformToGeometryIndex[TopLevelNode] = GeometryIdx;
				Collection.TransformIndex[GeometryIdx] = TopLevelNode;
				Collection.FaceCount[GeometryIdx] = 0;
				Collection.FaceStart[GeometryIdx] = Collection.Indices.Num();
				Collection.VertexCount[GeometryIdx] = 0;
				Collection.VertexStart[GeometryIdx] = Collection.Vertex.Num();
			}
		}
		Collection.SimulationType[TopLevelNode] = FGeometryCollection::ESimulationTypes::FST_Rigid;
		UpdateCollection.UpdateAllCollections(Collection);
	}

	TArray<int32> Children;
	for (int32 RmTransformIdx : AllRemoveIndices)
	{
		for (int32 ChildIdx : Collection.Children[RmTransformIdx])
		{
			Children.Add(ChildIdx);
		}
	}
	if (Children.Num())
	{
		GeometryCollectionAlgo::ParentTransforms(&Collection, TopLevelNode, Children);
	}

	// remove transforms for all geometry that was merged in
	FManagedArrayCollection::FProcessingParameters ProcessingParams;
#if !UE_BUILD_DEBUG
	ProcessingParams.bDoValidation = false;
#endif
	Collection.RemoveElements(FGeometryCollection::TransformGroup, AllRemoveIndices, ProcessingParams);
}


void RecomputeNormalsAndTangents(bool bOnlyTangents, bool bMakeSharpEdges, float SharpAngleDegrees, FGeometryCollection& Collection, const TArrayView<const int32>& TransformIndices,
								 bool bOnlyInternalSurfaces)
{
	FTransform CellsToWorld = FTransform::Identity;

	FDynamicMeshCollection MeshCollection(&Collection, TransformIndices, CellsToWorld, true);

	for (int MeshIdx = 0; MeshIdx < MeshCollection.Meshes.Num(); MeshIdx++)
	{
		FDynamicMesh3& Mesh = MeshCollection.Meshes[MeshIdx].AugMesh;
		AugmentedDynamicMesh::ComputeTangents(Mesh, bOnlyInternalSurfaces, !bOnlyTangents, bMakeSharpEdges, SharpAngleDegrees);
	}

	MeshCollection.UpdateAllCollections(Collection);

	Collection.ReindexMaterials();
}



int32 AddCollisionSampleVertices(double CollisionSampleSpacing, FGeometryCollection& Collection, const TArrayView<const int32>& TransformIndices)
{
	FTransform CellsToWorld = FTransform::Identity;

	FDynamicMeshCollection MeshCollection(&Collection, TransformIndices, CellsToWorld);

	MeshCollection.AddCollisionSamples(CollisionSampleSpacing);

	MeshCollection.UpdateAllCollections(Collection);

	Collection.ReindexMaterials();

	// TODO: This function does not create any new bones, so we could change it to not return anything
	return INDEX_NONE;
}

template <typename TransformType>
void ConvertToDynamicMeshTemplate(
	FDynamicMesh3& CombinedMesh,
	FTransform& TransformOut,
	bool bCenterPivot,
	const FGeometryCollection& Collection,
	TArrayView<const TransformType> BoneTransforms,
	TArrayView<const int32> TransformIndices,
	TFunction<int32(int32, bool)> RemapMaterialIDs,
	bool bClearCustomAttributes = true,
	bool bWeldEdges = true,
	bool bComponentSpaceTransforms = false,
	bool bAllowInvisible = true,
	bool bSetPolygroupPerBone = false
)
{
	FTransform CellsToWorld = FTransform::Identity;
	TransformOut = FTransform::Identity;

	FDynamicMeshCollection MeshCollection;
	MeshCollection.bSkipInvisible = !bAllowInvisible;
	MeshCollection.bComponentSpaceTransforms = bComponentSpaceTransforms && !BoneTransforms.IsEmpty();
	if (BoneTransforms.Num())
	{
		MeshCollection.Init(&Collection, BoneTransforms, TransformIndices, CellsToWorld);
	}
	else
	{
		MeshCollection.Init(&Collection, Collection.Transform, TransformIndices, CellsToWorld);
	}

	SetGeometryCollectionAttributes(CombinedMesh, Collection.NumUVLayers());
	CombinedMesh.Attributes()->EnableTangents();
	if (bSetPolygroupPerBone)
	{
		CombinedMesh.EnableTriangleGroups();
	}

	int32 NumMeshes = MeshCollection.Meshes.Num();
	for (int32 MeshIdx = 0; MeshIdx < NumMeshes; MeshIdx++)
	{
		FDynamicMesh3& Mesh = MeshCollection.Meshes[MeshIdx].AugMesh;
		if (bSetPolygroupPerBone)
		{
			Mesh.EnableTriangleGroups();
		}
		const FTransform& FromCollection = MeshCollection.Meshes[MeshIdx].FromCollection;

		FMeshNormals::InitializeOverlayToPerVertexNormals(Mesh.Attributes()->PrimaryNormals(), true);
		AugmentedDynamicMesh::InitializeOverlayToPerVertexUVs(Mesh, Collection.NumUVLayers());
		AugmentedDynamicMesh::InitializeOverlayToPerVertexTangents(Mesh);
		AugmentedDynamicMesh::InitializeOverlayToPerVertexColors(Mesh);

		if (RemapMaterialIDs)
		{
			FDynamicMeshMaterialAttribute* MaterialIDs = Mesh.Attributes()->GetMaterialID();
			for (int32 TID : Mesh.TriangleIndicesItr())
			{
				int32 MatID = MaterialIDs->GetValue(TID);
				bool bIsInternal = AugmentedDynamicMesh::GetInternal(Mesh, TID);
				int32 NewMatID = RemapMaterialIDs(MatID, bIsInternal);
				MaterialIDs->SetValue(TID, NewMatID);
			}
		}

		if (bWeldEdges && Mesh.TriangleCount() > 0)
		{
			FMergeCoincidentMeshEdges EdgeMerge(&Mesh);
			EdgeMerge.Apply();
		}

		if (MeshIdx > 0)
		{
			FDynamicMeshEditor MeshAppender(&CombinedMesh);
			FMeshIndexMappings IndexMaps_Unused;
			MeshAppender.AppendMesh(&Mesh, IndexMaps_Unused);
		}
		else
		{
			CombinedMesh = Mesh;
		}
	}

	if (bCenterPivot)
	{
		FAxisAlignedBox3d Bounds = CombinedMesh.GetBounds(true);
		FVector3d Translate = -Bounds.Center();
		MeshTransforms::Translate(CombinedMesh, Translate);
		TransformOut = FTransform((FVector)-Translate);
	}

	if (bClearCustomAttributes)
	{
		ClearCustomGeometryCollectionAttributes(CombinedMesh);
	}
}

template <typename TransformType>
void ConvertToMeshDescriptionTemplate(
	FMeshDescription& MeshOut,
	FTransform& TransformOut,
	bool bCenterPivot,
	FGeometryCollection& Collection,
	const TManagedArray<TransformType>& BoneTransforms,
	const TArrayView<const int32>& TransformIndices,
	TFunction<int32(int32, bool)> RemapMaterialIDs
)
{	
	FDynamicMesh3 CombinedMesh;
	ConvertToDynamicMeshTemplate<TransformType>(CombinedMesh, TransformOut, bCenterPivot, Collection, BoneTransforms.GetConstArray(), TransformIndices, RemapMaterialIDs, false);

	FDynamicMeshToMeshDescription Converter;
	Converter.Convert(&CombinedMesh, MeshOut, true);
}

void ConvertToMeshDescription(
	FMeshDescription& MeshOut,
	FTransform& TransformOut,
	bool bCenterPivot,
	FGeometryCollection& Collection,
	const TManagedArray<FTransform>& BoneTransforms,
	const TArrayView<const int32>& TransformIndices,
	TFunction<int32(int32, bool)> RemapMaterialIDs
)
{
	ConvertToMeshDescriptionTemplate(MeshOut, TransformOut, bCenterPivot, Collection, BoneTransforms, TransformIndices, RemapMaterialIDs);
}

void ConvertToMeshDescription(
	FMeshDescription& MeshOut,
	FTransform& TransformOut,
	bool bCenterPivot,
	FGeometryCollection& Collection,
	const TManagedArray<FTransform3f>& BoneTransforms,
	const TArrayView<const int32>& TransformIndices,
	TFunction<int32(int32, bool)> RemapMaterialIDs
)
{
	ConvertToMeshDescriptionTemplate(MeshOut, TransformOut, bCenterPivot, Collection, BoneTransforms, TransformIndices, RemapMaterialIDs);
}

void ConvertGeometryCollectionToDynamicMesh(
	FDynamicMesh3& OutputMesh,
	FTransform& TransformOut,
	bool bCenterPivot,
	const FGeometryCollection& Collection,
	bool bWeldEdges,
	TArrayView<const FTransform3f> BoneTransforms,
	bool bUseRelativeTransforms,
	TArrayView<const int32> TransformIndices,
	TFunction<int32(int32, bool)> RemapMaterialIDs,
	bool bAllowInvisible, bool bSetPolygroupPerBone
)
{
	ConvertToDynamicMeshTemplate<FTransform3f>(OutputMesh, TransformOut, bCenterPivot, Collection, BoneTransforms, TransformIndices, RemapMaterialIDs, true, bWeldEdges, !bUseRelativeTransforms, bAllowInvisible, bSetPolygroupPerBone);
}

#undef LOCTEXT_NAMESPACE
