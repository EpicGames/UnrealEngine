// Copyright Epic Games, Inc. All Rights Reserved.

#include "PlanarCutPlugin.h"



DEFINE_LOG_CATEGORY(LogPlanarCut);


class FPlanarCutPlugin : public IPlanarCutPlugin
{
	/** IModuleInterface implementation */
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
};

IMPLEMENT_MODULE( FPlanarCutPlugin, PlanarCut )



void FPlanarCutPlugin::StartupModule()
{
	
}


void FPlanarCutPlugin::ShutdownModule()
{
	
}



