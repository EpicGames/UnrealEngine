// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once


#include "Math/Vector4.h"
#include "Math/MathFwd.h"

class FGeometryCollection;
namespace UE::Geometry { struct FIndex4i; }
namespace UE::Geometry { template <typename PixelType> class TImageBuilder; }

class FProgressCancel;

namespace UE { namespace PlanarCut {


// Note: prefer ETargetFaces below
enum class EUseMaterials
{
	AllMaterials,		// Include all materials
	OddMaterials,		// Include materials w/ odd IDs (+ any manually selected materials)
	NoDefaultMaterials	// No default materials; Only use manually selected materials
};

enum class ETargetFaces
{
	AllFaces,
	InternalFaces,
	ExternalFaces,
	// If custom faces is chosen, then no faces are selected by default, and some other criteria (such as Material ID) must be used to select the target faces
	CustomFaces
};


/**
 * Box project UVs
 *
 * @param TargetUVLayer		Which UV layer to update
 * @param Collection		The collection to be box projected
 * @param BoxDimensions		Scale of projection box
 * @param TargetFaces		Which faces to automatically consider for box projection
 * @param TargetMaterials	If non-empty, additionally consider faces with listed material IDs for box projection
 */
bool PLANARCUT_API BoxProjectUVs(
	int32 TargetUVLayer,
	FGeometryCollection& Collection,
	const FVector3d& BoxDimensions,
	ETargetFaces TargetFaces = ETargetFaces::InternalFaces,
	TArrayView<int32> TargetMaterials = TArrayView<int32>(),
	FVector2f OffsetUVs = FVector2f(.5f,.5f),
	bool bOverrideBoxDimensionsWithBounds = false,
	bool bCenterBoxAtPivot = false,
	bool bUniformProjectionScale = false
);

/**
 * Box project UVs
 *
 * @param TargetUVLayer		Which UV layer to update
 * @param Collection		The collection to be box projected
 * @param BoxDimensions		Scale of projection box
 * @param MaterialsPattern	Which pattern of material IDs to automatically consider for box projection
 * @param WhichMaterials	If non-empty, consider listed material IDs for UV box projection
 */
UE_DEPRECATED(5.3, "Use the BoxProjectUVs variant that takes ETargetFaces instead")
inline bool BoxProjectUVs(
	int32 TargetUVLayer,
	FGeometryCollection& Collection,
	const FVector3d& BoxDimensions,
	EUseMaterials MaterialsPattern = EUseMaterials::OddMaterials,
	TArrayView<int32> WhichMaterials = TArrayView<int32>(),
	FVector2f OffsetUVs = FVector2f(.5f,.5f),
	bool bOverrideBoxDimensionsWithBounds = false,
	bool bCenterBoxAtPivot = false,
	bool bUniformProjectionScale = false
)
{
	ETargetFaces TargetFaces =
		MaterialsPattern == EUseMaterials::AllMaterials ? ETargetFaces::AllFaces :
		MaterialsPattern == EUseMaterials::OddMaterials ? ETargetFaces::InternalFaces : ETargetFaces::CustomFaces;
	return BoxProjectUVs(TargetUVLayer, Collection, BoxDimensions, TargetFaces, WhichMaterials, OffsetUVs, bOverrideBoxDimensionsWithBounds, bCenterBoxAtPivot, bUniformProjectionScale);
}

struct FMergeIslandSettings
{
	//
	// Parameters for island merging
	//

	// Threshold for allowed area distortion from merging islands (when we use ExpMap to compute new UVs for the merged island)
	double AreaDistortionThreshold = 1.5;
	// Threshold for allowed normal deviation between merge-able islands
	double MaxNormalDeviationDeg = 45.0;

	//
	// ExpMap parameters, for computing new UVs of merged islands
	//

	int32 NormalSmoothingRounds = 0;
	double NormalSmoothingAlpha = 0.25;
};

/**
 * Merge existing UV islands on the chosen faces and UV Layer, based on a normal angle threshold and a distortion threshold.
 * The Exponential Map algorithm will be used to compute new UVs for the merged islands.
 */
bool PLANARCUT_API MergeUVIslands(
	int32 TargetUVLayer,
	FGeometryCollection& Collection,
	FMergeIslandSettings MergeIslandSettings,
	TArrayView<bool> FaceSelection,
	FProgressCancel* Progress = nullptr
);


/**
 * Make a UV atlas of non-overlapping UV charts for a geometry collection, using the existing UV islands
 *
 * @param TargetUVLayer		Which UV layer to update with new UV coordinates
 * @param Collection		The collection to be atlas'd
 * @param UVRes				Target resolution for the atlas
 * @param GutterSize		Space to leave between UV islands, in pixels at the target resolution
 * @param TargetFaces		Which faces to automatically consider for UV island layout
 * @param TargetMaterials	If non-empty, additionally consider faces with listed material IDs for UV island layout
 * @param bRecreateUVsForDegenerateIslands If true, detect and fix islands that don't have proper UVs (i.e. UVs all zero or otherwise collapsed to a point)
 */
bool PLANARCUT_API UVLayout(
	int32 TargetUVLayer,
	FGeometryCollection& Collection,
	int32 UVRes = 1024,
	float GutterSize = 1,
	ETargetFaces MaterialsPattern = ETargetFaces::InternalFaces,
	TArrayView<int32> WhichMaterials = TArrayView<int32>(),
	bool bRecreateUVsForDegenerateIslands = true,
	FProgressCancel* Progress = nullptr
);

/**
 * Make a UV atlas of non-overlapping UV charts for a geometry collection ( face selection version )
 *
 * @param TargetUVLayer		Which UV layer to update with new UV coordinates
 * @param Collection		The collection to be atlas'd
 * @param UVRes				Target resolution for the atlas
 * @param GutterSize		Space to leave between UV islands, in pixels at the target resolution
 * @param FacesSelection	Which faces to automatically consider for UV island layout
  * @param bRecreateUVsForDegenerateIslands If true, detect and fix islands that don't have proper UVs (i.e. UVs all zero or otherwise collapsed to a point)
 */
bool PLANARCUT_API UVLayout(
	int32 TargetUVLayer,
	FGeometryCollection& Collection,
	int32 UVRes,
	float GutterSize,
	TArrayView<bool> FaceSelection,
	bool bRecreateUVsForDegenerateIslands = true,
	FProgressCancel* Progress = nullptr
);

/**
 * Make a UV atlas of non-overlapping UV charts for a geometry collection
 *
 * @param TargetUVLayer		Which UV layer to update with new UV coordinates
 * @param Collection		The collection to be atlas'd
 * @param UVRes				Target resolution for the atlas
 * @param GutterSize		Space to leave between UV islands, in pixels at the target resolution
 * @param MaterialsPattern	Which pattern of material IDs to automatically consider for UV island layout
 * @param WhichMaterials	If non-empty, consider listed material IDs for UV island layout
 * @param bRecreateUVsForDegenerateIslands If true, detect and fix islands that don't have proper UVs (i.e. UVs all zero or otherwise collapsed to a point)
 */
UE_DEPRECATED(5.3, "Use the UVLayout variant that takes ETargetFaces instead")
inline bool UVLayout(
	int32 TargetUVLayer,
	FGeometryCollection& Collection,
	int32 UVRes = 1024,
	float GutterSize = 1,
	EUseMaterials MaterialsPattern = EUseMaterials::OddMaterials,
	TArrayView<int32> WhichMaterials = TArrayView<int32>(),
	bool bRecreateUVsForDegenerateIslands = true,
	FProgressCancel* Progress = nullptr
)
{
	ETargetFaces TargetFaces =
		MaterialsPattern == EUseMaterials::AllMaterials ? ETargetFaces::AllFaces :
		MaterialsPattern == EUseMaterials::OddMaterials ? ETargetFaces::InternalFaces : ETargetFaces::CustomFaces;
	return UVLayout(TargetUVLayer, Collection, UVRes, GutterSize, TargetFaces, WhichMaterials, bRecreateUVsForDegenerateIslands, Progress);
}


// Different attributes we can bake
enum class EBakeAttributes : int32
{
	None,
	DistanceToExternal,
	AmbientOcclusion,
	Curvature,
	NormalX,
	NormalY,
	NormalZ,
	PositionX,
	PositionY,
	PositionZ
};

struct FTextureAttributeSettings
{
	double ToExternal_MaxDistance = 100.0;
	int AO_Rays = 32;
	double AO_BiasAngleDeg = 15.0;
	bool bAO_Blur = true;
	double AO_BlurRadius = 2.5;
	double AO_MaxDistance = 0.0; // 0.0 is interpreted as TNumericLimits<double>::Max()
	int Curvature_VoxelRes = 128;
	double Curvature_Winding = .5;
	int Curvature_SmoothingSteps = 10;
	double Curvature_SmoothingPerStep = .8;
	bool bCurvature_Blur = true;
	double Curvature_BlurRadius = 2.5;
	double Curvature_ThicknessFactor = 3.0; // distance to search for mesh correspondence, as a factor of voxel size
	double Curvature_MaxValue = .1; // curvatures above this value will be clamped
	int ClearGutterChannel = -1; // don't copy gutter values for this channel, if specified -- useful for visualizing the UV island borders
};


/**
 * Generate a texture for specified groups of faces based on chosen BakeAttributes and AttributeSettings
 *
 * @param TargetUVLayer		Which UV layer to take UV coordinates from when creating the new texture
 * @param Collection		The collection to be create a new texture for
 * @param GutterSize		Number of texels to fill outside of UV island borders (values are copied from nearest inside pt)
 * @param BakeAttributes	Which attributes to bake into which color channel
 * @param AttributeSettings	Settings for the BakeAttributes
 * @param TextureOut		Texture to write to
 * @param MaterialsPattern	Which pattern of material IDs to apply texture to
 * @param WhichMaterials	If non-empty, apply texture to the listed material IDs
 */
bool PLANARCUT_API TextureSpecifiedFaces(
	int32 TargetUVLayer,
	FGeometryCollection& Collection,
	int32 GutterSize,
	UE::Geometry::FIndex4i BakeAttributes,
	const FTextureAttributeSettings& AttributeSettings,
	UE::Geometry::TImageBuilder<FVector4f>& TextureOut,
	ETargetFaces MaterialsPattern = ETargetFaces::InternalFaces,
	TArrayView<int32> WhichMaterials = TArrayView<int32>(),
	FProgressCancel* Progress = nullptr
);

/**
 * Generate a texture for specified groups of faces based on chosen BakeAttributes and AttributeSettings
 *
 * @param TargetUVLayer		Which UV layer to take UV coordinates from when creating the new texture
 * @param Collection		The collection to be create a new texture for
 * @param GutterSize		Number of texels to fill outside of UV island borders (values are copied from nearest inside pt)
 * @param BakeAttributes	Which attributes to bake into which color channel
 * @param AttributeSettings	Settings for the BakeAttributes
 * @param TextureOut		Texture to write to
 * @param ToTextureTriangles boolean Array that indicate what face from the collection is to be baked
 * @param WhichMaterials	If non-empty, apply texture to the listed material IDs
 */
bool PLANARCUT_API TextureSpecifiedFaces(
	int32 TargetUVLayer,
	FGeometryCollection& Collection,
	int32 GutterSize,
	UE::Geometry::FIndex4i BakeAttributes,
	const FTextureAttributeSettings& AttributeSettings,
	UE::Geometry::TImageBuilder<FVector4f>& TextureOut,
	TArrayView<bool> ToTextureTriangles,
	FProgressCancel* Progress = nullptr
);

UE_DEPRECATED(5.3, "Use TextureSpecifiedFaces instead")
bool PLANARCUT_API TextureInternalSurfaces(
	int32 TargetUVLayer,
	FGeometryCollection& Collection,
	int32 GutterSize,
	UE::Geometry::FIndex4i BakeAttributes,
	const FTextureAttributeSettings& AttributeSettings,
	UE::Geometry::TImageBuilder<FVector4f>& TextureOut,
	EUseMaterials MaterialsPattern = EUseMaterials::OddMaterials,
	TArrayView<int32> WhichMaterials = TArrayView<int32>(),
	FProgressCancel* Progress = nullptr
);

}} // namespace UE::PlanarCut
