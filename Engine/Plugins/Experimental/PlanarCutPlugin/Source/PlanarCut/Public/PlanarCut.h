// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once


#include "GeometryCollection/GeometryCollection.h"
#include "DynamicMesh/DynamicMesh3.h"

#define UE_API PLANARCUT_API

class FVoronoiDiagram;
namespace GeometryCollection::Facades { class FCollectionMeshFacade; }
struct FMeshDescription;

class FProgressCancel;

// Offsets to randomize where Perlin noise is sampled for each dimension of a noise vector
struct FNoiseOffsets
{
	FNoiseOffsets(FRandomStream& RandomStream) 
		: X(RandomStream.VRand() * RandomOffsetScale), Y(RandomStream.VRand() * RandomOffsetScale), Z(RandomStream.VRand() * RandomOffsetScale)
	{
	}

	FNoiseOffsets() {}

	void SetOffsets(FRandomStream& RandomStream)
	{
		X = RandomStream.VRand() * RandomOffsetScale;
		Y = RandomStream.VRand() * RandomOffsetScale;
		Z = RandomStream.VRand() * RandomOffsetScale;
	}

	FVector X, Y, Z;
private:
	// Offset scale chosen to keep reasonably small offsets while still de-correlating the noise in each dimension.
	static constexpr double RandomOffsetScale = 100.0;
};

struct FNoiseSettings
{
	// if NoiseOverride is set, will be used instead of default Octave Noise.
	TFunction<FVector3d(const FVector& Pos)> NoiseOverride = nullptr;
	// desired spacing between noise samples, used for both Octave Noise and Noise Override cases
	float PointSpacing = 1;

	// Octave Noise settings, not used if NoiseOverride is specified
	float Amplitude = 2;
	float Frequency = 0.1f;
	int32 Octaves = 4;
	float Lacunarity = 2;
	float Persistence = 0.5f;

	FVector NoiseVector(const FVector& Pos, const FNoiseOffsets& Offsets) const
	{
		if (NoiseOverride)
		{
			return NoiseOverride(Pos);
		}

		FVector Base = Pos * Frequency;
		return FVector(
			OctaveNoise(Base + Offsets.X),
			OctaveNoise(Base + Offsets.Y),
			OctaveNoise(Base + Offsets.Z)
		) * Amplitude;
	}

	float OctaveNoise(const FVector& V) const
	{
		float NoiseValue = 0;
		float FreqScale = 1;
		float AmpScale = 1;
		for (int32 Octave = 0; Octave < Octaves; Octave++, FreqScale *= Lacunarity, AmpScale *= Persistence)
		{
			NoiseValue += FMath::PerlinNoise3D(V * FreqScale) * AmpScale;
		}
		return NoiseValue;
	}
};

/** Settings controlling how disconnected islands are detected and split after a cut */
struct FIslandSplitSettings
{

	FIslandSplitSettings(bool bInSplitIslands = true, double InCloseVertexDistance = 1e-3, double InVertexToSurfaceBridgeDistance = 0)
		: bSplitIslands(bInSplitIslands), CloseVertexDistance(InCloseVertexDistance),
		  VertexToSurfaceBridgeDistance(InVertexToSurfaceBridgeDistance) {}

	/** Whether to detect and split disconnected islands */
	bool bSplitIslands = true;

	/** Distance threshold for connecting spatially-close vertices before computing islands */
	double CloseVertexDistance = 1e-3;

	/** If > 0, bridge separate islands whose surfaces are within this vertex-to-triangle distance */
	double VertexToSurfaceBridgeDistance = 0;

};

// auxiliary structure for FPlanarCells to carry material info
struct FInternalSurfaceMaterials
{
	int32 GlobalMaterialID = 0;
	bool bGlobalVisibility = true;
	float GlobalUVScale = 1;

	TOptional<FNoiseSettings> NoiseSettings; // if unset, noise will not be added

	// TODO: add optional overrides per facet / per cell
	
	/**
	 * @param Collection	Reference collection to use for setting UV scale
	 * @param GeometryIdx	Reference geometry inside collection; if -1, use all geometries in collection
	 */
	UE_API void SetUVScaleFromCollection(const GeometryCollection::Facades::FCollectionMeshFacade& CollectionMesh, int32 GeometryIdx = -1);

	
	UE_API int32 GetDefaultMaterialIDForGeometry(const FGeometryCollection& Collection, int32 GeometryIdx = -1) const;
};

// Stores planar facets that divide space into cells
struct FPlanarCells
{
	FPlanarCells()
	{
	}
	UE_API FPlanarCells(const FPlane& Plane);
	UE_API FPlanarCells(const TArrayView<const FVector> Sites, FVoronoiDiagram &Voronoi);
	UE_API FPlanarCells(const TArrayView<const FBox> Boxes, bool bResolveAdjacencies = false);
	UE_API FPlanarCells(const FBox &Region, const FIntVector& CubesPerAxis);
	UE_API FPlanarCells(const FBox &Region, const TArrayView<const FColor> Image, int32 Width, int32 Height);

	int32 NumCells;
	bool AssumeConvexCells = false; // enables optimizations in this common case (can create incorrect geometry if set to true when cells are not actually convex)
	TArray<FPlane> Planes;

	TArray<TPair<int32, int32>> PlaneCells;  // the two cells neighboring each plane, w/ the cell on the negative side of the plane first, positive side second
	TArray<TArray<int32>> PlaneBoundaries;
	TArray<FVector> PlaneBoundaryVertices;

	FInternalSurfaceMaterials InternalSurfaceMaterials;

	// Discards cells from the diagram, used to support visualization of a subset of cells
	UE_API void DiscardCells(TFunctionRef<bool(int32)> KeepFunc, bool bKeepNeighbors);

	/**
	 * @return true if this is a single, unbounded cutting plane
	 */
	bool IsInfinitePlane() const
	{
		return NumCells == 2 && Planes.Num() == 1 && PlaneBoundaries[0].Num() == 0;
	}
	
	/**
	 * Debugging function to check that the plane boundary vertices are wound to match the orientation of the plane normal vectors
	 * @return	false if any plane boundary vertices are found in the 'wrong' orientation relative to the plane normal
	 */
	bool HasValidPlaneBoundaryOrientations() const
	{
		for (int32 PlaneIdx = 0; PlaneIdx < PlaneBoundaries.Num(); PlaneIdx++)
		{
			const TArray<int32>& Bdry = PlaneBoundaries[PlaneIdx];
			if (Bdry.Num() < 3)
			{
				continue;
			}
			const FPlane &P = Planes[PlaneIdx];
			FVector N(P.X, P.Y, P.Z);
			if (!N.IsNormalized()) // plane normals should be normalized
			{
				return false;
			}
			const FVector& A = PlaneBoundaryVertices[Bdry[0]];
			const FVector& B = PlaneBoundaryVertices[Bdry[1]];
			const FVector& C = PlaneBoundaryVertices[Bdry[2]];
			FVector E1 = B - A;
			FVector E2 = C - B;
			FVector NormalDir = E2 ^ E1;
			
			for (int32 VIdx : Bdry)
			{
				const double SD = P.PlaneDot(PlaneBoundaryVertices[VIdx]);
				if (FMath::Abs(SD) > 1e-4)
				{
					return false; // vertices should be on plane!
				}
			}
			if (AssumeConvexCells && FVector::DotProduct(NormalDir, N) < 0) // vectors aren't pointing the same way at all?
			{
				return false;
			}
			const double AngleMeasure = (NormalDir ^ N).SizeSquared();
			if (AngleMeasure > 1e-3) // vectors aren't directionally aligned?
			{
				return false;
			}
		}
		return true;
	}

	inline void AddPlane(const FPlane &P, int32 CellIdxBehind, int32 CellIdxInFront)
	{
		Planes.Add(P);
		PlaneCells.Emplace(CellIdxBehind, CellIdxInFront);
		PlaneBoundaries.Emplace();
	}

	inline int32 AddPlane(const FPlane &P, int32 CellIdxBehind, int32 CellIdxInFront, const TArray<int32>& PlaneBoundary)
	{
		int32 PlaneIdx = Planes.Add(P);
		PlaneCells.Emplace(CellIdxBehind, CellIdxInFront);
		PlaneBoundaries.Add(PlaneBoundary);
		return PlaneIdx;
	}

	void SetNoise(FNoiseSettings Noise = FNoiseSettings())
	{
		InternalSurfaceMaterials.NoiseSettings = Noise;
	}
};


/**
 * Cut a Geometry inside a GeometryCollection with PlanarCells, and add each cut cell back to the GeometryCollection as a new child of the input Geometry.  For geometries that would not be cut, nothing is added.
 * 
 * @param Cells				Defines the cutting planes and division of space
 * @param Collection		The collection to be cut
 * @param TransformIdx		Which transform inside the collection to cut
 * @param Grout				Separation to leave between cutting cells
 * @param CollisionSampleSpacing	Target spacing between collision sample vertices
 * @param RandomSeed				Seed to be used for random noise displacement
 * @param TransformCollection		Optional transform of the whole geometry collection; if unset, defaults to Identity
 * @param bIncludeOutsideCellInOutput	If true, geometry that was not inside any of the cells (e.g. was outside of the bounds of all cutting geometry) will still be included in the output; if false, it will be discarded.
 * @param Progress						Optionally tracks progress and supports early-cancel
 * @param CellsOrigin					Optionally provide a local origin of the cutting Cells
 * @param IslandSplitSettings			Controls detection and splitting of disconnected components after the cut
 * @return	index of first new geometry in the Output GeometryCollection, or -1 if no geometry was added
 */
int32 PLANARCUT_API CutWithPlanarCells(
	FPlanarCells &Cells,
	FGeometryCollection& Collection,
	int32 TransformIdx,
	double Grout,
	double CollisionSampleSpacing,
	int32 RandomSeed,
	const TOptional<FTransform>& TransformCollection = TOptional<FTransform>(),
	bool bIncludeOutsideCellInOutput = true,
	bool bSetDefaultInternalMaterialsFromCollection = true,
	FProgressCancel* Progress = nullptr,
	FVector CellsOrigin = FVector::ZeroVector,
	FIslandSplitSettings IslandSplitSettings = FIslandSplitSettings()
);

/**
 * Generate cutting mesh surfaces as a single mesh
 * Useful for creating a preview of the cut surface.
 * 
 * @param Cells				Defines the cutting planes and division of space
 * @param Grout				Separation to leave between cutting cells
 * @param RandomSeed				Seed to be used for random noise displacement
 * @param OutCuttingMeshes			Dynamic mesh representing the preview surface
 * @param FilterCellsFunc			Filter which cells should be included in the preview
 * @param TransformCollection		Optional transform that would be applied to the to-cut surface; if unset, defaults to Identity
 * @param Progress						Optionally tracks progress and supports early-cancel
 * @param CellsOrigin					Optionally provide a local origin of the cutting Cells
 */
void PLANARCUT_API CreateCuttingSurfacePreview(
	const FPlanarCells& Cells,
	const FBox& Bounds,
	double Grout,
	int32 RandomSeed,
	UE::Geometry::FDynamicMesh3& OutCuttingMeshes,
	TFunctionRef<bool(int)> FilterCellsFunc,
	const TOptional<FTransform>& TransformCollection = TOptional<FTransform>(),
	FProgressCancel* Progress = nullptr,
	FVector CellsOrigin = FVector::ZeroVector
);

/**
 * Cut multiple Geometry groups inside a GeometryCollection with PlanarCells, and add each cut cell back to the GeometryCollection as a new child of their source Geometry.  For geometries that would not be cut, nothing is added.
 *
 * @param Cells				Defines the cutting planes and division of space
 * @param Collection		The collection to be cut
 * @param TransformIndices	Which transform groups inside the collection to cut
 * @param Grout				Separation to leave between cutting cells
 * @param CollisionSampleSpacing	Target spacing between collision sample vertices
 * @param RandomSeed				Seed to be used for random noise displacement
 * @param TransformCollection		Optional transform of the whole geometry collection; if unset, defaults to Identity
 * @param bIncludeOutsideCellInOutput	If true, geometry that was not inside any of the cells (e.g. was outside of the bounds of all cutting geometry) will still be included in the output; if false, it will be discarded.
 * @param Progress						Optionally tracks progress and supports early-cancel
 * @param CellsOrigin					Optionally provide a local origin of the cutting Cells
 * @param IslandSplitSettings			Controls detection and splitting of disconnected components after the cut
 * @return	index of first new geometry in the Output GeometryCollection, or -1 if no geometry was added
 */
int32 PLANARCUT_API CutMultipleWithPlanarCells(
	FPlanarCells &Cells,
	FGeometryCollection& Collection,
	const TArrayView<const int32>& TransformIndices,
	double Grout,
	double CollisionSampleSpacing,
	int32 RandomSeed,
	const TOptional<FTransform>& TransformCollection = TOptional<FTransform>(),
	bool bIncludeOutsideCellInOutput = true,
	bool bSetDefaultInternalMaterialsFromCollection = true,
	FProgressCancel* Progress = nullptr,
	FVector CellsOrigin = FVector::ZeroVector,
	FIslandSplitSettings IslandSplitSettings = FIslandSplitSettings()
);

/**
 * Split the geometry at the given transforms into their connected components
 *
 * @param Collection		The collection to be cut
 * @param TransformIndices	Which transform groups inside the collection to cut
 * @param CollisionSampleSpacing	Target spacing between collision sample vertices
 * @param Progress						Optionally tracks progress and supports early-cancel
 * @return	index of first new geometry in the Output GeometryCollection, or -1 if no geometry was added
 */
int32 PLANARCUT_API SplitIslands(
	FGeometryCollection& Collection,
	const TArrayView<const int32>& TransformIndices,
	double CollisionSampleSpacing,
	FProgressCancel* Progress = nullptr,
	FIslandSplitSettings IslandSplitSettings = FIslandSplitSettings()
);

/**
 * Cut multiple Geometry groups inside a GeometryCollection with Planes, and add each cut cell back to the GeometryCollection as a new child of their source Geometry.  For geometries that would not be cut, nothing is added.
 *
 * @param Planes				Defines the cutting planes and division of space
 * @param InternalSurfaceMaterials	Defines material properties for any added internal surfaces
 * @param Collection			The collection to be cut
 * @param TransformIndices	Which transform groups inside the collection to cut
 * @param Grout				Separation to leave between cutting cells
 * @param CollisionSampleSpacing	Target spacing between collision sample vertices
 * @param RandomSeed				Seed to be used for random noise displacement
 * @param TransformCollection		Optional transform of the whole geometry collection; if unset, defaults to Identity
 * @param Progress					Optionally tracks progress and supports early-cancel
 * @param IslandSplitSettings			Controls detection and splitting of disconnected components after the cut
 * @return	index of first new geometry in the Output GeometryCollection, or -1 if no geometry was added
 */
int32 PLANARCUT_API CutMultipleWithMultiplePlanes(
	const TArrayView<const FPlane>& Planes,
	FInternalSurfaceMaterials& InternalSurfaceMaterials,
	FGeometryCollection& Collection,
	const TArrayView<const int32>& TransformIndices,
	double Grout,
	double CollisionSampleSpacing,
	int32 RandomSeed,
	const TOptional<FTransform>& TransformCollection = TOptional<FTransform>(),
	bool bSetDefaultInternalMaterialsFromCollection = true,
	FProgressCancel* Progress = nullptr,
	FIslandSplitSettings IslandSplitSettings = FIslandSplitSettings()
);


/**
 * Populate an array of transform indices w/ those that are smaller than a threshold volume
 *
 * @param Collection			The collection to be processed
 * @param TransformIndices		The transform indices to process, or empty if all should be processed
 * @param OutVolumes			Output array, to be filled w/ volumes of geometry; 1:1 w/ TransformIndices array
 * @param ScalePerDimension		Scale to apply per dimension (e.g. 1/100 converts volume from centimeters^3 to meters^3)
 * @param bIncludeClusters		Whether to set the volumes for clusters as a sum of the child volumes. Otherwise, cluster volumes are set to zero.
 */
void PLANARCUT_API FindBoneVolumes(
	FGeometryCollection& Collection,
	const TArrayView<const int32>& TransformIndices,
	TArray<double>& OutVolumes,
	double ScalePerDimension = .01,
	bool bIncludeClusters = false
);

/**
 * Populate an array of transform indices w/ those that are smaller than a threshold volume
 *
 * @param Collection			The collection to be processed
 * @param TransformIndices		The transform indices to process, or empty if all should be processed
 * @param Volumes				Volumes of geometry; 1:1 w/ TransformIndices array
 * @param MinVolume				Geometry smaller than this quantity will be chosen
 * @param OutSmallBones			Output array, to be filled with transform indices for small pieces of geometry
 * @param bIncludeClusters		Whether to include clusters in the search
 */
void PLANARCUT_API FindSmallBones(
	const FGeometryCollection& Collection,
	const TArrayView<const int32>& TransformIndices,
	const TArrayView<const double>& Volumes,
	double MinVolume,
	TArray<int32>& OutSmallBones,
	bool bIncludeClusters = false
);

/**
 * Populate an array of transform indices w/ those that match a custom volume-based filter
 *
 * @param Collection			The collection to be processed
 * @param TransformIndices		The transform indices to process, or empty if all should be processed
 * @param Volumes				Volumes of geometry; 1:1 w/ TransformIndices array
 * @param Filter				Geometry for which the volume filter returns true will be chosen
 * @param OutSmallBones			Output array, to be filled with transform indices for small pieces of geometry
 * @param bIncludeClusters		Whether to allow clusters in the filter
 */
void PLANARCUT_API FilterBonesByVolume(
	const FGeometryCollection& Collection,
	const TArrayView<const int32>& TransformIndices,
	const TArrayView<const double>& Volumes,
	TFunctionRef<bool(double Volume, int32 BoneIdx)> Filter,
	TArray<int32>& OutSmallBones,
	bool bIncludeClusters = false
);

namespace UE
{
	namespace PlanarCut
	{
		enum ENeighborSelectionMethod
		{
			LargestNeighbor,
			NearestCenter,
			LargestContactArea
		};
	}
}


/**
 * Merge chosen geometry into neighboring geometry.
 *
 * @param Collection				The collection to be processed
 * @param TransformIndices			The transform indices to process, or empty if all should be processed
 * @param Volumes					Volumes of geometry; 1:1 w/ TransformIndices array
 * @param MinVolume					If merged small geometry is larger than this, it will not require further merging
 * @param SmallTransformIndices		Transform indices of pieces that we want to merge
 * @param bUnionJoinedPieces		Try to 'union' the merged pieces, removing internal triangles and connecting the shared cut boundary
 * @param NeighborSelectionMethod	How to choose which neighbor to merge to
 * @param bUseCollectionProximity	If true, will use the proximity data on the collection to decide which pieces are allowed to merge. Otherwise, a reasonable default proximity will be computed and used.
 * @param bOnlySameParent			Whether to only merge nodes that have the same parent in the hierarchy
 */
int32 PLANARCUT_API MergeBones(
	FGeometryCollection& Collection,
	const TArrayView<const int32>& TransformIndices,
	const TArrayView<const double>& Volumes,
	double MinVolume,
	const TArrayView<const int32>& SmallTransformIndices,
	bool bUnionJoinedPieces,
	UE::PlanarCut::ENeighborSelectionMethod NeighborSelectionMethod,
	bool bUseCollectionProximity = true,
	bool bOnlySameParent = false
);

/**
 * Merge chosen bones into neighboring clusters.
 *
 * @param Collection				The collection to be processed
 * @param Volumes					Volumes of geometry; 1:1 w/ TransformIndices array
 * @param MinVolume					If merged small geometry is larger than this, it will not require further merging
 * @param SmallTransformIndices		Transform indices of pieces that we want to merge
 * @param NeighborSelectionMethod	How to choose which neighbor to merge to
 * @param bOnlyMergeInProximity		Whether to only follow proximity graph, or to fall back to nearest center when for when no proximity links are found
 * @param bOnlySameParent			Whether to only merge nodes that have the same parent in the hierarchy
 * @param bUseCollectionProximity	If true, will use the proximity data on the collection to decide which pieces are allowed to merge. Otherwise, a reasonable default proximity will be computed and used.
 */
void PLANARCUT_API MergeClusters(
	FGeometryCollection& Collection,
	const TArrayView<const double>& Volumes,
	double MinVolume,
	const TArrayView<const int32>& SmallTransformIndices,
	UE::PlanarCut::ENeighborSelectionMethod NeighborSelectionMethod,
	bool bOnlyMergeInProximity,
	bool bOnlySameParent,
	bool bUseCollectionProximity = true
);

/**
 * Merge all chosen nodes into a single node.  Unlike MergeBones(), does not account for proximity and will always only merge the selected bones.
 *
 * @param Collection				The collection to be processed
 * @param TransformIndices			The transform indices to process, or empty if all should be processed
 * @param bUnionJoinedPieces		Try to 'union' the merged pieces, removing internal triangles and connecting the shared cut boundary
 */
void PLANARCUT_API MergeAllSelectedBones(
	FGeometryCollection& Collection,
	const TArrayView<const int32>& TransformIndices,
	bool bUnionJoinedPieces
);

/**
 * Recompute normals and tangents of selected geometry, optionally restricted to faces with odd or given material IDs (i.e. to target internal faces)
 *
 * @param bOnlyTangents		If true, leave normals unchanged and only recompute tangent&bitangent vectors
 * @param bMakeSharpEdges	If true, recompute the normal topology to split normals at 'sharp' edges
 * @param SharpAngleDegrees	If bMakeSharpEdges, edges w/ adjacent triangle normals deviating by more than this threshold will be sharp edges (w/ split normals)
 * @param Collection		The Geometry Collection to be updated
 * @param TransformIndices	Which transform groups on the Geometry Collection to be updated.  If empty, all groups are updated.
 * @param bOnlyOddMaterials	If true, restrict recomputation to odd-numbered material IDs
 * @param WhichMaterials	If non-empty, restrict recomputation to only the listed material IDs
 */
void PLANARCUT_API RecomputeNormalsAndTangents(bool bOnlyTangents, bool bMakeSharpEdges, float SharpAngleDegrees, FGeometryCollection& Collection, const TArrayView<const int32>& TransformIndices = TArrayView<const int32>(),
	bool bOnlyInternalSurfaces = true);
/**
 * Scatter additional vertices (w/ no associated triangle) as needed to satisfy minimum point spacing
 * 
 * @param TargetSpacing		The desired spacing between collision sample vertices
 * @param Collection		The Geometry Collection to be updated
 * @param TransformIndices	Which transform groups on the Geometry Collection to be updated.  If empty, all groups are updated.
 * @return Index of first transform group w/ updated geometry.  (To update geometry we delete and re-add, because geometry collection isn't designed for in-place updates)
 */
int32 PLANARCUT_API AddCollisionSampleVertices(double TargetSpacing, FGeometryCollection& Collection, const TArrayView<const int32>& TransformIndices = TArrayView<const int32>());

/**
 * Cut multiple Geometry groups inside a GeometryCollection with a mesh, and add each cut cell back to the GeometryCollection as a new child of their source Geometry.  For geometries that would not be cut, nothing is added.
 * 
 * @param CuttingMesh				Mesh to be used to cut the geometry collection
 * @param CuttingMeshTransform		Position of cutting mesh
 * @param InternalSurfaceMaterials	Defines material properties for any added internal surfaces
 * @param Collection				The collection to be cut
 * @param TransformIndices			Which transform groups inside the collection to cut
 * @param CollisionSampleSpacing	Target spacing between collision sample vertices
 * @param TransformCollection		Optional transform of the collection; if unset, defaults to Identity
 * @param Progress					Optionally tracks progress and supports early-cancel
 * @param IslandSplitSettings			Controls detection and splitting of disconnected components after the cut
 * @return index of first new geometry in the Output GeometryCollection, or -1 if no geometry was added
 */
int32 PLANARCUT_API CutWithMesh(
	const UE::Geometry::FDynamicMesh3& CuttingMesh,
	FTransform CuttingMeshTransform,
	FInternalSurfaceMaterials& InternalSurfaceMaterials,
	FGeometryCollection& Collection,
	const TArrayView<const int32>& TransformIndices,
	double CollisionSampleSpacing,
	const TOptional<FTransform>& TransformCollection = TOptional<FTransform>(),
	bool bSetDefaultInternalMaterialsFromCollection = true,
	FProgressCancel* Progress = nullptr,
	FIslandSplitSettings IslandSplitSettings = FIslandSplitSettings()
);

/// Convert a mesh description to a dynamic mesh *specifically* augmented / designed for cutting geometry collections, to be passed to CutWithMesh
UE::Geometry::FDynamicMesh3 PLANARCUT_API ConvertMeshDescriptionToCuttingDynamicMesh(const FMeshDescription* CuttingMesh, int32 NumUVLayers, FProgressCancel* Progress = nullptr);

inline int32 CutWithMesh(
	const FMeshDescription* CuttingMesh,
	FTransform CuttingMeshTransform,
	FInternalSurfaceMaterials& InternalSurfaceMaterials,
	FGeometryCollection& Collection,
	const TArrayView<const int32>& TransformIndices,
	double CollisionSampleSpacing,
	const TOptional<FTransform>& TransformCollection = TOptional<FTransform>(),
	bool bSetDefaultInternalMaterialsFromCollection = true,
	FProgressCancel* Progress = nullptr,
	FIslandSplitSettings IslandSplitSettings = FIslandSplitSettings()
)
{
	int32 NumUVLayers = Collection.NumUVLayers();
	return CutWithMesh(ConvertMeshDescriptionToCuttingDynamicMesh(CuttingMesh, NumUVLayers, Progress),
		CuttingMeshTransform,
		InternalSurfaceMaterials,
		Collection,
		TransformIndices,
		CollisionSampleSpacing,
		TransformCollection,
		bSetDefaultInternalMaterialsFromCollection,
		Progress,
		IslandSplitSettings
		);
}



/**
 * Convert chosen Geometry groups inside a GeometryCollection to a single Mesh Description.
 *
 * @param OutputMesh				Mesh to be filled with the geometry collection geometry
 * @param TransformOut				Transform taking output mesh geometry to local space of geometry collection
 * @param bCenterPivot				Whether to center the geometry at the origin
 * @param Collection				The collection to be converted
 * @param TransformIndices			Which transform groups inside the collection to convert
 * @param RemapMaterialIDs			Optional function to remap (MaterialID, bIsInternal) -> NewMaterialID
 */
void PLANARCUT_API ConvertToMeshDescription(
	FMeshDescription& OutputMesh,
	FTransform& TransformOut,
	bool bCenterPivot,
	FGeometryCollection& Collection,
	const TManagedArray<FTransform>& BoneTransforms,
	const TArrayView<const int32>& TransformIndices,
	TFunction<int32(int32, bool)> RemapMaterialIDs = nullptr
);

/**
 * Convert chosen Geometry groups inside a GeometryCollection to a single Mesh Description.
 *
 * @param OutputMesh				Mesh to be filled with the geometry collection geometry
 * @param TransformOut				Transform taking output mesh geometry to local space of geometry collection
 * @param bCenterPivot				Whether to center the geometry at the origin
 * @param Collection				The collection to be converted
 * @param TransformIndices			Which transform groups inside the collection to convert
 * @param RemapMaterialIDs			Optional function to remap (MaterialID, bIsInternal) -> NewMaterialID
 */
void PLANARCUT_API ConvertToMeshDescription(
	FMeshDescription& OutputMesh,
	FTransform& TransformOut,
	bool bCenterPivot,
	FGeometryCollection& Collection,
	const TManagedArray<FTransform3f>& BoneTransforms,
	const TArrayView<const int32>& TransformIndices,
	TFunction<int32(int32, bool)> RemapMaterialIDs = nullptr
);

/**
 * Convert chosen Geometry groups inside a GeometryCollection to a single Dynamic Mesh.
 *
 * @param OutputMesh				Mesh to be filled with the geometry collection geometry
 * @param TransformOut				Transform taking output mesh geometry to local space of geometry collection. This will be Identity unless bCenterPivot is true.
 * @param bCenterPivot				Whether to center the geometry at the origin
 * @param Collection				The collection to be converted
 * @param bWeldEdges				Whether to weld edges on conversion
 * @param BoneTransforms			Transforms for each bone in the geometry collection
 * @param bUseRelativeTransforms	Whether the BoneTransforms are relative to each bone's parent, or in a shared space (e.g., Collection.Transform is relative, UGeometryCollectionComponent's component space transforms are not)
 * @param TransformIndices			Which transform groups inside the collection to convert
 * @param RemapMaterialIDs			Optional function to remap (MaterialID, bIsInternal) -> NewMaterialID
 * @param bAllowInvisible			Allow mesh to include triangles that are not marked 'visible' (typically, pre-fracture geometry stored for potential history/LOD usage)
 * @param bSetPolygroupPerBone		Enable PolyGroups on the OutputMesh and set one group per geometry
 */
void PLANARCUT_API ConvertGeometryCollectionToDynamicMesh(
	UE::Geometry::FDynamicMesh3& OutputMesh,
	FTransform& TransformOut,
	bool bCenterPivot,
	const FGeometryCollection& Collection,
	bool bWeldEdges,
	TArrayView<const FTransform3f> BoneTransforms,
	bool bUseRelativeTransforms,
	TArrayView<const int32> TransformIndices,
	TFunction<int32(int32, bool)> RemapMaterialIDs = nullptr,
	bool bAllowInvisible = false,
	bool bSetPolygroupPerBone = true
);

#undef UE_API
